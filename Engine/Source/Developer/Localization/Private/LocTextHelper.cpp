// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocTextHelper.h"

#include "Dom/JsonObject.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/IBreakIterator.h"
#include "LocalizationFileUtil.h"
#include "Logging/StructuredLog.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "PlatformInfo.h"
#include "Serialization/Csv/CsvParser.h"
#include "Serialization/JsonInternationalizationArchiveSerializer.h"
#include "Serialization/JsonInternationalizationManifestSerializer.h"
#include "Serialization/JsonInternationalizationMetadataSerializer.h"
#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocTextHelper)

#define LOCTEXT_NAMESPACE "LocTextHelper"

DEFINE_LOG_CATEGORY_STATIC(LogLocTextHelper, Log, All);
namespace LocTextHelper
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

bool FLocTextPlatformSplitUtils::ShouldSplitPlatformData(const ELocTextPlatformSplitMode& InSplitMode)
{
	return InSplitMode != ELocTextPlatformSplitMode::None;
}

const TArray<FString>& FLocTextPlatformSplitUtils::GetPlatformsToSplit(const ELocTextPlatformSplitMode& InSplitMode)
{
	switch (InSplitMode)
	{
	case ELocTextPlatformSplitMode::Confidential:
		{
			static const TArray<FString> ConfidentialPlatformNames = []()
			{
				TArray<FString> TmpArray;
				for (FName Name : FDataDrivenPlatformInfoRegistry::GetConfidentialPlatforms())
				{
					TmpArray.Add(Name.ToString());
				}
				TmpArray.Sort();
				return TmpArray;
			}();
			return ConfidentialPlatformNames;
		}

	case ELocTextPlatformSplitMode::All:
		{
			static const TArray<FString> AllPlatformNames = []()
			{
				// the Keys of the FDataDrivenPlatformInfoRegistry platforms are the known ini platform names
				TArray<FString> TmpArray;
				for (FName Name : FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType::AllPlatformInfos))
				{
					TmpArray.Add(Name.ToString());
				}
				return TmpArray;
			}();
			return AllPlatformNames;
		}

	default:
		break;
	}

	static TArray<FString> EmptyArray;
	return EmptyArray;
}


void FLocTextConflicts::AddConflict(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadata, const FLocItem& InSource, const FString& InSourceLocation)
{
	TSharedPtr<FConflict> ExistingEntry = FindEntryByKey(InNamespace, InKey, InKeyMetadata);
	if (!ExistingEntry.IsValid())
	{
		TSharedRef<FConflict> NewEntry = MakeShared<FConflict>(InNamespace, InKey, InKeyMetadata);
		EntriesByKey.Add(InKey, NewEntry);
		ExistingEntry = NewEntry;
	}
	ExistingEntry->Add(InSource, InSourceLocation.ReplaceCharWithEscapedChar());
}

TSharedPtr<FLocTextConflicts::FConflict> FLocTextConflicts::FindEntryByKey(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadata) const
{
	TArray<TSharedRef<FConflict>> MatchingEntries;
	EntriesByKey.MultiFind(InKey, MatchingEntries);

	for (const TSharedRef<FConflict>& Entry : MatchingEntries)
	{
		if (Entry->Namespace == InNamespace)
		{
			if (InKeyMetadata.IsValid() != Entry->KeyMetadataObj.IsValid())
			{
				continue;
			}
			else if ((!InKeyMetadata.IsValid() && !Entry->KeyMetadataObj.IsValid()) || (*InKeyMetadata == *Entry->KeyMetadataObj))
			{
				return Entry;
			}
		}
	}

	return nullptr;
}

FString FLocTextConflicts::GetConflictReport() const
{
	FString Report;

	for (const auto& ConflictPair : EntriesByKey)
	{
		const TSharedRef<FConflict>& Conflict = ConflictPair.Value;
		const FString& Namespace = Conflict->Namespace.GetString();
		const FString& Key = Conflict->Key.GetString();

		bool bAddToReport = false;
		// Grab the list of all conflicting source text associated with this key 
		TArray<FLocItem> SourceList;
		Conflict->EntriesBySourceLocation.GenerateValueArray(SourceList);

		if (SourceList.Num() >= 2)
		{
			for (int32 i = 0; i < SourceList.Num() - 1 && !bAddToReport; ++i)
			{
				// we only add the conflict to the report if the source strings do not match up.
				for (int32 j = i + 1; j < SourceList.Num() && !bAddToReport; ++j)
				{
					if (!(SourceList[i] == SourceList[j]))
					{
						bAddToReport = true;
					}
				}
			}
		}

		if (bAddToReport)
		{
			FString KeyMetadataString = FJsonInternationalizationMetaDataSerializer::MetadataToString(Conflict->KeyMetadataObj);
			Report += FString::Printf(TEXT("%s - %s %s\n"), *Namespace, *Key, *KeyMetadataString);

			for (auto EntryIter = Conflict->EntriesBySourceLocation.CreateConstIterator(); EntryIter; ++EntryIter)
			{
				const FString& SourceLocation = EntryIter.Key();
				FString ProcessedSourceLocation = FPaths::ConvertRelativePathToFull(SourceLocation);
				ProcessedSourceLocation.ReplaceInline(TEXT("\\"), TEXT("/"));
				ProcessedSourceLocation.ReplaceInline(*FPaths::RootDir(), TEXT("/"));

				const FString& SourceText = EntryIter.Value().Text.ReplaceCharWithEscapedChar();

				FString SourceMetadataString = FJsonInternationalizationMetaDataSerializer::MetadataToString(EntryIter.Value().MetadataObj);
				Report += FString::Printf(TEXT("\t%s - \"%s\" %s\n"), *ProcessedSourceLocation, *SourceText, *SourceMetadataString);
			}
			Report += TEXT("\n");
		}
	}

	return Report;
}

FString FLocTextConflicts::GetConflictReportAsCSV() const
{
	// First have the headers 
	FString Report = TEXT("Namespace,Key,File1,Line1,File2,Line2,Source1,Source2\n");

	for (const auto& ConflictPair : EntriesByKey)
	{
		const TSharedRef<FConflict>& Conflict = ConflictPair.Value;
		const FString& Namespace = Conflict->Namespace.GetString();
		const FString& Key = Conflict->Key.GetString();

		bool bAddToReport = false;
		TArray<FLocItem> SourceList;
		Conflict->EntriesBySourceLocation.GenerateValueArray(SourceList);
		if (SourceList.Num() >= 2)
		{
			for (int32 i = 0; i < SourceList.Num() - 1 && !bAddToReport; ++i)
			{
				for (int32 j = i + 1; j < SourceList.Num() && !bAddToReport; ++j)
				{
					// Conflict only if the source text and metadata don't match 
					if (!(SourceList[i] == SourceList[j]))
					{
						bAddToReport = true;
					}
				}
			}
		}

		if (bAddToReport)
		{
			// the format of the CSV will need file1,line1 and file2,line2 to be compared against each other 
			// If a single key has more than 1 conflict, we want file 1 in the csv to always be the same one.
			// Thus we store the results of the first entry to be appended in each row of the CSV for a single entry 

			bool bFirstEntry = true;
			FString FirstEntrySourceFile;
			FString FirstEntryLineNumber;
			FString FirstEntrySourceText;

			// This maps each of the source locations to its conflicting source string 
			// for each entry in this, we create a new row in the .csv 
			for (auto EntryIter = Conflict->EntriesBySourceLocation.CreateConstIterator(); EntryIter; ++EntryIter)
			{
				const FString& SourceLocation = EntryIter.Key();
				FString ProcessedSourceLocation = SourceLocation;
				ProcessedSourceLocation.ReplaceInline(TEXT("\\"), TEXT("/"));

				const FString& SourceText = EntryIter.Value().Text.ReplaceCharWithEscapedChar();
				// in the event there are more than 2 key conflicts for a single key,
				// this will always be the entry for file 1 and line 1 in the output file 
				if (bFirstEntry)
				{
					// For a source file conflict, source location is in the format My/File/Name.cpp:LineNumber
					bool bSplitSuccessful = ProcessedSourceLocation.Split(TEXT(":"), &FirstEntrySourceFile, &FirstEntryLineNumber, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					// If the split fails, this means that the location is not in the form file:linenumber.
					// if the line number string is not numeric, this means we are dealing with an asset path 
					if (!bSplitSuccessful  || !FirstEntryLineNumber.IsNumeric())
					{
						FirstEntrySourceFile = ProcessedSourceLocation;
						FirstEntryLineNumber = TEXT("0");
					}
					FirstEntrySourceText = SourceText;
					bFirstEntry = false;
					continue;
				}
				FString CurrentSourceFile;
				FString CurrentLineNumber;
				bool bSplitSuccessful = ProcessedSourceLocation.Split(TEXT(":"), &CurrentSourceFile, &CurrentLineNumber, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				// Same as above 
				if (!bSplitSuccessful || !CurrentLineNumber.IsNumeric())
				{
					CurrentSourceFile = ProcessedSourceLocation;
					CurrentLineNumber = TEXT("0");
				}
				Report += FString::Printf(TEXT("%s,%s,%s,%s,%s,%s,%s,%s\n"), *Namespace, *Key, *FirstEntrySourceFile, *FirstEntryLineNumber, *CurrentSourceFile, *CurrentLineNumber, *FirstEntrySourceText, *SourceText);
			}
		}
	}
	return Report;
}

const FString FLocTextWordCounts::ColHeadingDateTime = TEXT("Date/Time");
const FString FLocTextWordCounts::ColHeadingWordCount = TEXT("Word Count");

void FLocTextWordCounts::FRowData::ResetWordCounts()
{
	SourceWordCount = 0;
	PerCultureWordCounts.Reset();
}

bool FLocTextWordCounts::FRowData::IdenticalWordCounts(const FRowData& InOther) const
{
	if (SourceWordCount != InOther.SourceWordCount)
	{
		return false;
	}

	if (PerCultureWordCounts.Num() != InOther.PerCultureWordCounts.Num())
	{
		return false;
	}

	for (const auto& PerCultureWordCountPair : PerCultureWordCounts)
	{
		int32 OtherPerCultureWordCount = 0;
		if (const int32* FoundOtherPerCultureWordCount = InOther.PerCultureWordCounts.Find(PerCultureWordCountPair.Key))
		{
			OtherPerCultureWordCount = *FoundOtherPerCultureWordCount;
		}

		if (OtherPerCultureWordCount != PerCultureWordCountPair.Value)
		{
			return false;
		}
	}

	return true;
}

FLocTextWordCounts::FRowData& FLocTextWordCounts::AddRow(int32* OutIndex)
{
	const int32 RowIndex = Rows.AddDefaulted();
	if (OutIndex)
	{
		*OutIndex = RowIndex;
	}
	return Rows[RowIndex];
}

FLocTextWordCounts::FRowData* FLocTextWordCounts::GetRow(const int32 InIndex)
{
	return Rows.IsValidIndex(InIndex) ? &Rows[InIndex] : nullptr;
}

const FLocTextWordCounts::FRowData* FLocTextWordCounts::GetRow(const int32 InIndex) const
{
	return Rows.IsValidIndex(InIndex) ? &Rows[InIndex] : nullptr;
}

int32 FLocTextWordCounts::GetRowCount() const
{
	return Rows.Num();
}

void FLocTextWordCounts::TrimReport()
{
	SortRowsByDate();

	for (int32 RowIndex = 1; RowIndex < Rows.Num(); ++RowIndex)
	{
		const FRowData& PreviousRowData = Rows[RowIndex - 1];
		const FRowData& CurrentRowData = Rows[RowIndex];
		if (PreviousRowData.IdenticalWordCounts(CurrentRowData))
		{
			Rows.RemoveAt(RowIndex--, EAllowShrinking::No);
			continue;
		}
	}
}

bool FLocTextWordCounts::FromCSV(const FString& InCSVString, FText* OutError)
{
	const FCsvParser CsvParser(InCSVString);
	const auto& CsvRows = CsvParser.GetRows();

	// Must have at least 2 rows (timestamp + source word count)
	if (CsvRows.Num() <= 1)
	{
		if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_WordCountsFromCSV_TooFewRows", "Failed to parse the CSV string as it contained too few rows (expected at least 2, got {0})."), CsvRows.Num());
		}
		return false;
	}

	int32 DateTimeColumn = INDEX_NONE;
	int32 WordCountColumn = INDEX_NONE;
	TMap<FString, int32> PerCultureColumns;

	// Make sure our header has the required columns
	{
		const TArray<const TCHAR*>& CsvCells = CsvRows[0];

		for (int32 CellIdx = 0; CellIdx < CsvCells.Num(); ++CellIdx)
		{
			const TCHAR* Cell = CsvCells[CellIdx];
			if (FCString::Stricmp(Cell, *ColHeadingDateTime) == 0 && DateTimeColumn == INDEX_NONE)
			{
				DateTimeColumn = CellIdx;
			}
			else if (FCString::Stricmp(Cell, *ColHeadingWordCount) == 0 && WordCountColumn == INDEX_NONE)
			{
				WordCountColumn = CellIdx;
			}
			else
			{
				PerCultureColumns.Add(Cell, CellIdx);
			}
		}

		const bool bValidHeader = DateTimeColumn != INDEX_NONE && WordCountColumn != INDEX_NONE;
		if (!bValidHeader)
		{
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("Error_WordCountsFromCSV_InvalidHeader", "Failed to parse the CSV string as the header was missing one of the required rows (either '{0}' or '{1}')."), FText::FromString(ColHeadingDateTime), FText::FromString(ColHeadingWordCount));
			}
			return false;
		}
	}

	// Perform the import
	Rows.Reset(CsvRows.Num() - 1);
	for (int32 CsvRowIdx = 1; CsvRowIdx < CsvRows.Num(); ++CsvRowIdx)
	{
		const TArray<const TCHAR*>& CsvCells = CsvRows[CsvRowIdx];

		// Must have at least an entry for the required columns
		if (CsvCells.IsValidIndex(DateTimeColumn) && CsvCells.IsValidIndex(WordCountColumn))
		{
			FRowData& RowData = Rows[Rows.AddDefaulted()];

			// Parse required data
			FDateTime::Parse(CsvCells[DateTimeColumn], RowData.Timestamp);
			LexFromString(RowData.SourceWordCount, CsvCells[WordCountColumn]);

			// Parse per-culture data
			for (const auto& PerCultureColumnPair : PerCultureColumns)
			{
				if (CsvCells.IsValidIndex(PerCultureColumnPair.Value))
				{
					int32 PerCultureWordCount = 0;
					LexFromString(PerCultureWordCount, CsvCells[PerCultureColumnPair.Value]);
					RowData.PerCultureWordCounts.Add(PerCultureColumnPair.Key, PerCultureWordCount);
				}
			}
		}
	}

	return true;
}

FString FLocTextWordCounts::ToCSV()
{
	SortRowsByDate();

	// Collect and sort the per-culture column names used by any row
	TArray<FString> PerCultureColumnNames;
	for (const FRowData& RowData : Rows)
	{
		for (const auto& PerCultureWordCountPair : RowData.PerCultureWordCounts)
		{
			PerCultureColumnNames.AddUnique(PerCultureWordCountPair.Key);
		}
	}
	PerCultureColumnNames.Sort();

	FString CSVString;

	// Write the header
	{
		CSVString += ColHeadingDateTime;
		CSVString += TEXT(",");
		CSVString += ColHeadingWordCount;
		for (const FString& PerCultureColumnName : PerCultureColumnNames)
		{
			CSVString += TEXT(",");
			CSVString += PerCultureColumnName;
		}
		CSVString += TEXT("\n");
	}

	// Write each row
	for (const FRowData& RowData : Rows)
	{
		CSVString += RowData.Timestamp.ToString();
		CSVString += TEXT(",");
		CSVString += FString::Printf(TEXT("%d"), RowData.SourceWordCount);
		for (const FString& PerCultureColumnName : PerCultureColumnNames)
		{
			int32 PerCultureWordCount = 0;
			if (const int32* FoundPerCultureWordCount = RowData.PerCultureWordCounts.Find(PerCultureColumnName))
			{
				PerCultureWordCount = *FoundPerCultureWordCount;
			}

			CSVString += TEXT(",");
			CSVString += FString::Printf(TEXT("%d"), PerCultureWordCount);
		}
		CSVString += TEXT("\n");
	}

	return CSVString;
}

void FLocTextWordCounts::SortRowsByDate()
{
	Rows.Sort([](const FRowData& InOne, const FRowData& InTwo)
	{
		return InOne.Timestamp < InTwo.Timestamp;
	});
}


FLocTextHelper::FLocTextHelper(FString InTargetName, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode)
	: PlatformSplitMode(InPlatformSplitMode)
	, TargetName(MoveTemp(InTargetName))
	, LocFileNotifies(MoveTemp(InLocFileNotifies))
{
}

FLocTextHelper::FLocTextHelper(FString InTargetPath, FString InManifestName, FString InArchiveName, FString InNativeCulture, TArray<FString> InForeignCultures, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode)
	: PlatformSplitMode(InPlatformSplitMode)
	, TargetPath(MoveTemp(InTargetPath))
	, ManifestName(MoveTemp(InManifestName))
	, ArchiveName(MoveTemp(InArchiveName))
	, NativeCulture(MoveTemp(InNativeCulture))
	, ForeignCultures(MoveTemp(InForeignCultures))
	, LocFileNotifies(MoveTemp(InLocFileNotifies))
{
	checkf(!TargetPath.IsEmpty(), TEXT("Target path may not be empty!"));
	checkf(!ManifestName.IsEmpty(), TEXT("Manifest name may not be empty!"));
	checkf(!ArchiveName.IsEmpty(), TEXT("Archive name may not be empty!"));

	// todo: We currently infer the target name from the manifest, however once all target files are named consistently the target name should be passed in rather than the manifest/archive names
	TargetName = FPaths::GetBaseFilename(ManifestName);

	// Make sure the native culture isn't in the list of foreign cultures
	if (!NativeCulture.IsEmpty())
	{
		ForeignCultures.Remove(NativeCulture);
	}
}

FLocTextHelper::FLocTextHelper(TSharedPtr<FInternationalizationManifest> InLoadedManifest, FString InTargetName, FString InTargetPath, FString InNativeCulture, TArray<FString> InForeignCultures, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode)
	: FLocTextHelper(MoveTemp(InTargetPath),
		InTargetName + TEXT(".manifest"),
		InTargetName + TEXT(".archive"),
		MoveTemp(InNativeCulture),
		MoveTemp(InForeignCultures),
		MoveTemp(InLocFileNotifies),
		InPlatformSplitMode)
{
	TargetName = MoveTemp(InTargetName);
	Manifest = MoveTemp(InLoadedManifest);
}

bool FLocTextHelper::ShouldSplitPlatformData() const
{
	return FLocTextPlatformSplitUtils::ShouldSplitPlatformData(PlatformSplitMode);
}

ELocTextPlatformSplitMode FLocTextHelper::GetPlatformSplitMode() const
{
	return PlatformSplitMode;
}

const TArray<FString>& FLocTextHelper::GetPlatformsToSplit() const
{
	return FLocTextPlatformSplitUtils::GetPlatformsToSplit(PlatformSplitMode);
}

const FString& FLocTextHelper::GetTargetName() const
{
	return TargetName;
}

const FString& FLocTextHelper::GetTargetPath() const
{
	return TargetPath;
}

TSharedPtr<ILocFileNotifies> FLocTextHelper::GetLocFileNotifies() const
{
	return LocFileNotifies;
}

void FLocTextHelper::SetCopyrightNotice(const FString& InCopyrightNotice)
{
	CopyrightNotice = InCopyrightNotice;
}

const FString& FLocTextHelper::GetCopyrightNotice() const
{
	return CopyrightNotice;
}

const FString& FLocTextHelper::GetNativeCulture() const
{
	return NativeCulture;
}

const TArray<FString>& FLocTextHelper::GetForeignCultures() const
{
	return ForeignCultures;
}

TArray<FString> FLocTextHelper::GetAllCultures(const bool bSingleCultureMode) const
{
	// Single-culture mode is a hack for the Localization commandlets
	// In this mode we only include the native culture if we have no foreign cultures
	const bool bIncludeNativeCulture = (!bSingleCultureMode || ForeignCultures.Num() == 0) && !NativeCulture.IsEmpty();

	TArray<FString> AllCultures;
	if (bIncludeNativeCulture)
	{
		AllCultures.Add(NativeCulture);
	}
	AllCultures.Append(ForeignCultures);
	return AllCultures;
}

bool FLocTextHelper::HasManifest() const
{
	return Manifest.IsValid();
}

bool FLocTextHelper::LoadManifest(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	const FString ManifestFilePath = TargetPath / ManifestName;
	return LoadManifest(ManifestFilePath, InLoadFlags, OutError);
}

bool FLocTextHelper::LoadManifest(const FString& InManifestFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	Manifest.Reset();
	Manifest = LoadManifestImpl(InManifestFilePath, InLoadFlags, OutError);
	return Manifest.IsValid();
}

bool FLocTextHelper::SaveManifest(FText* OutError) const
{
	const FString ManifestFilePath = TargetPath / ManifestName;
	return SaveManifest(ManifestFilePath, OutError);
}

bool FLocTextHelper::SaveManifest(const FString& InManifestFilePath, FText* OutError) const
{
	if (!Manifest.IsValid())
	{
		if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_SaveManifest_NoManifest", "Failed to save file '{0}' as there is no manifest instance to save."), FText::FromString(InManifestFilePath));
		}
		return false;
	}

	return SaveManifestImpl(Manifest.ToSharedRef(), InManifestFilePath, OutError);
}

void FLocTextHelper::TrimManifest()
{
	if (Dependencies.Num() > 0)
	{
		// We'll generate a new manifest by only including items that are not in the dependencies
		TSharedRef<FInternationalizationManifest> TrimmedManifest = MakeShared<FInternationalizationManifest>();

		for (FManifestEntryByStringContainer::TConstIterator It(Manifest->GetEntriesBySourceTextIterator()); It; ++It)
		{
			const TSharedRef<FManifestEntry> ManifestEntry = It.Value();

			for (const FManifestContext& Context : ManifestEntry->Contexts)
			{
				FString DependencyFileName;
				TSharedPtr<FManifestEntry> DependencyEntry = FindDependencyEntry(ManifestEntry->Namespace, Context, &DependencyFileName);

				// Ignore this dependency if the platforms are different
				if (DependencyEntry.IsValid())
				{
					const FManifestContext* DependencyContext = DependencyEntry->FindContext(Context.Key, Context.KeyMetadataObj);
					if (Context.PlatformName != DependencyContext->PlatformName)
					{
						DependencyEntry.Reset();
						DependencyFileName.Reset();
					}
				}

				if (DependencyEntry.IsValid())
				{
					if (!(DependencyEntry->Source.IsExactMatch(ManifestEntry->Source)))
					{
						const FManifestContext* ConflictingContext = DependencyEntry->FindContext(Context.Key, Context.KeyMetadataObj);
						const FString DependencyEntryFullSrcLoc = (!DependencyFileName.IsEmpty()) ? DependencyFileName : ConflictingContext->SourceLocation;

						// There is a dependency manifest entry that has the same namespace and keys as our main manifest entry but the source text differs.
						UE_LOGFMT(LogLocTextHelper, Warning, "Text conflict for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. Main manifest entry is '{text}'{textMetaData}. Conflict from manifest dependency is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.",
							("locNamespace", *SanitizeLogOutput(ManifestEntry->Namespace.GetString())),
							("locKey", *SanitizeLogOutput(Context.Key.GetString())),
							("locKeyMetaData", (Context.KeyMetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(Context.KeyMetadataObj))) : TEXT(""))),
							("text", *SanitizeLogOutput(ManifestEntry->Source.Text)),
							("textMetaData", (ManifestEntry->Source.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(ManifestEntry->Source.MetadataObj))) : TEXT(""))),
							("conflictLocation", *DependencyFileName),
							("conflictText", *SanitizeLogOutput(DependencyEntry->Source.Text)),
							("conflictTextMetaData", (DependencyEntry->Source.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(DependencyEntry->Source.MetadataObj))) : TEXT(""))),
							("id", LocTextHelper::LocalizationLogIdentifier)
						);
						
						ConflictTracker.AddConflict(ManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, ManifestEntry->Source, *Context.SourceLocation);
						ConflictTracker.AddConflict(ManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, DependencyEntry->Source, DependencyEntryFullSrcLoc);
					}
				}
				else
				{
					// Since we did not find any entries in the dependencies list that match, we'll add to the new manifest
					const bool bAddSuccessful = TrimmedManifest->AddSource(ManifestEntry->Namespace, ManifestEntry->Source, Context);
					if (!bAddSuccessful)
					{
						UE_LOGFMT(LogLocTextHelper, Error, "Failed to add text for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData} from manifest dependency '{file}'.",
							("file", *DependencyFileName),
							("locNamespace", *ManifestEntry->Namespace.GetString()),
							("locKey", *Context.Key.GetString()),
							("locKeyMetaData", (Context.KeyMetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(Context.KeyMetadataObj)) : TEXT(""))),
							("text", *ManifestEntry->Source.Text),
							("textMetaData", (ManifestEntry->Source.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(ManifestEntry->Source.MetadataObj)) : TEXT(""))),
							("id", LocTextHelper::LocalizationLogIdentifier)
						);
					}
				}
			}
		}

		Manifest = TrimmedManifest;
	}
}

void FLocTextHelper::SortManifest()
{
	checkf(Manifest.IsValid(), TEXT("Attempted to enumerate source texts, but no manifest has been loaded!"));
	FJsonInternationalizationManifestSerializer::SortManifest(Manifest.ToSharedRef());
}

bool FLocTextHelper::HasNativeArchive() const
{
	return HasArchive(NativeCulture);
}

bool FLocTextHelper::LoadNativeArchive(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	return LoadArchive(NativeCulture, InLoadFlags, OutError);
}

bool FLocTextHelper::LoadNativeArchive(const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	return LoadArchive(NativeCulture, InArchiveFilePath, InLoadFlags, OutError);
}

bool FLocTextHelper::SaveNativeArchive(FText* OutError) const
{
	return SaveArchive(NativeCulture, OutError);
}

bool FLocTextHelper::SaveNativeArchive(const FString& InArchiveFilePath, FText* OutError) const
{
	return SaveArchive(NativeCulture, InArchiveFilePath, OutError);
}

bool FLocTextHelper::HasForeignArchive(const FString& InCulture) const
{
	checkf(ForeignCultures.Contains(InCulture), TEXT("Attempted to check for a foreign culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	return HasArchive(InCulture);
}

bool FLocTextHelper::LoadForeignArchive(const FString& InCulture, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	checkf(ForeignCultures.Contains(InCulture), TEXT("Attempted to load a foreign culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	return LoadArchive(InCulture, InLoadFlags, OutError);
}

bool FLocTextHelper::LoadForeignArchive(const FString& InCulture, const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	checkf(ForeignCultures.Contains(InCulture), TEXT("Attempted to load a foreign culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	return LoadArchive(InCulture, InArchiveFilePath, InLoadFlags, OutError);
}

bool FLocTextHelper::SaveForeignArchive(const FString& InCulture, FText* OutError) const
{
	checkf(ForeignCultures.Contains(InCulture), TEXT("Attempted to load a foreign culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	return SaveArchive(InCulture, OutError);
}

bool FLocTextHelper::SaveForeignArchive(const FString& InCulture, const FString& InArchiveFilePath, FText* OutError) const
{
	checkf(ForeignCultures.Contains(InCulture), TEXT("Attempted to load a foreign culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	return SaveArchive(InCulture, InArchiveFilePath, OutError);
}

bool FLocTextHelper::HasArchive(const FString& InCulture) const
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	return Archive.IsValid();
}

bool FLocTextHelper::LoadArchive(const FString& InCulture, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	const FString ArchiveFilePath = TargetPath / InCulture / ArchiveName;
	return LoadArchive(InCulture, ArchiveFilePath, InLoadFlags, OutError);
}

bool FLocTextHelper::LoadArchive(const FString& InCulture, const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	const bool bIsNativeArchive = !NativeCulture.IsEmpty() && InCulture == NativeCulture;
	const bool bIsForeignArchive = ForeignCultures.Contains(InCulture);
	checkf(bIsNativeArchive || bIsForeignArchive, TEXT("Attempted to load a culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);
	checkf(Manifest.IsValid(), TEXT("Attempted to load a culture archive file, but no manifest has been loaded!"));

	Archives.Remove(InCulture);

	TSharedPtr<FInternationalizationArchive> Archive = LoadArchiveImpl(InArchiveFilePath, InLoadFlags, OutError);
	if (Archive.IsValid())
	{
		Archives.Add(InCulture, Archive);
		return true;
	}
	return false;
}

bool FLocTextHelper::SaveArchive(const FString& InCulture, FText* OutError) const
{
	const FString ArchiveFilePath = TargetPath / InCulture / ArchiveName;
	return SaveArchive(InCulture, ArchiveFilePath, OutError);
}

bool FLocTextHelper::SaveArchive(const FString& InCulture, const FString& InArchiveFilePath, FText* OutError) const
{
	const bool bIsNativeArchive = !NativeCulture.IsEmpty() && InCulture == NativeCulture;
	const bool bIsForeignArchive = ForeignCultures.Contains(InCulture);
	checkf(bIsNativeArchive || bIsForeignArchive, TEXT("Attempted to save a culture archive file, but the given culture (%s) wasn't set during construction!"), *InCulture);

	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	if (!Archive.IsValid())
	{
		if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_SaveArchive_NoArchive", "Failed to save file '{0}' as there is no archive instance to save."), FText::FromString(InArchiveFilePath));
		}
		return false;
	}

	return SaveArchiveImpl(Archive.ToSharedRef(), InArchiveFilePath, OutError);
}

bool FLocTextHelper::LoadAllArchives(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	if (!NativeCulture.IsEmpty() && !LoadNativeArchive(InLoadFlags, OutError))
	{
		return false;
	}

	for (const FString& Culture : ForeignCultures)
	{
		if (!LoadForeignArchive(Culture, InLoadFlags, OutError))
		{
			return false;
		}
	}

	return true;
}

bool FLocTextHelper::SaveAllArchives(FText* OutError) const
{
	if (!NativeCulture.IsEmpty() && !SaveNativeArchive(OutError))
	{
		return false;
	}

	for (const FString& Culture : ForeignCultures)
	{
		if (!SaveForeignArchive(Culture, OutError))
		{
			return false;
		}
	}

	return true;
}

void FLocTextHelper::TrimArchive(const FString& InCulture)
{
	checkf(Manifest.IsValid(), TEXT("Attempted to trim an archive file, but no manifest has been loaded!"));

	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to trim an archive file, but no valid archive could be found for '%s'!"), *InCulture);

	TSharedPtr<FInternationalizationArchive> NativeArchive;
	if (!NativeCulture.IsEmpty() && InCulture != NativeCulture)
	{
		NativeArchive = Archives.FindRef(NativeCulture);
		checkf(NativeArchive.IsValid(), TEXT("Attempted to trim an archive file, but no valid archive could be found for '%s'!"), *NativeCulture);
	}

	// Copy any translations that match current manifest entries over into the trimmed archive
	TSharedRef<FInternationalizationArchive> TrimmedArchive = MakeShared<FInternationalizationArchive>();
	EnumerateSourceTexts([&](TSharedRef<FManifestEntry> InManifestEntry) -> bool
	{
		for (const FManifestContext& Context : InManifestEntry->Contexts)
		{
			// Keep any translation for the source text
			TSharedPtr<FArchiveEntry> ArchiveEntry = Archive->FindEntryByKey(InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj);
			if (ArchiveEntry.IsValid())
			{
				TrimmedArchive->AddEntry(ArchiveEntry.ToSharedRef());
			}
		}

		return true; // continue enumeration
	}, true);

	Archives[InCulture] = TrimmedArchive;
}

bool FLocTextHelper::LoadAll(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	if (!LoadManifest(InLoadFlags, OutError))
	{
		return false;
	}

	return LoadAllArchives(InLoadFlags, OutError);
}

bool FLocTextHelper::SaveAll(FText* OutError) const
{
	if (!SaveManifest(OutError))
	{
		return false;
	}

	return SaveAllArchives(OutError);
}

bool FLocTextHelper::AddDependency(const FString& InDependencyFilePath, FText* OutError)
{
	if (DependencyPaths.Contains(InDependencyFilePath))
	{
		return true;
	}

	TSharedPtr<FInternationalizationManifest> DepManifest = LoadManifestImpl(InDependencyFilePath, ELocTextHelperLoadFlags::Load, OutError);
	if (DepManifest.IsValid())
	{
		DependencyPaths.Add(InDependencyFilePath);
		Dependencies.Add(DepManifest);
		return true;
	}

	return false;
}

TSharedPtr<FManifestEntry> FLocTextHelper::FindDependencyEntry(const FLocKey& InNamespace, const FLocKey& InKey, const FString* InSourceText, FString* OutDependencyFilePath) const
{
	for (int32 DepIndex = 0; DepIndex < Dependencies.Num(); ++DepIndex)
	{
		TSharedPtr<FInternationalizationManifest> DepManifest = Dependencies[DepIndex];

		const TSharedPtr<FManifestEntry> DepEntry = DepManifest->FindEntryByKey(InNamespace, InKey, InSourceText);
		if (DepEntry.IsValid())
		{
			if (OutDependencyFilePath)
			{
				*OutDependencyFilePath = DependencyPaths[DepIndex];
			}
			return DepEntry;
		}
	}

	return nullptr;
}

TSharedPtr<FManifestEntry> FLocTextHelper::FindDependencyEntry(const FLocKey& InNamespace, const FManifestContext& InContext, FString* OutDependencyFilePath) const
{
	for (int32 DepIndex = 0; DepIndex < Dependencies.Num(); ++DepIndex)
	{
		TSharedPtr<FInternationalizationManifest> DepManifest = Dependencies[DepIndex];

		const TSharedPtr<FManifestEntry> DepEntry = DepManifest->FindEntryByContext(InNamespace, InContext);
		if (DepEntry.IsValid())
		{
			if (OutDependencyFilePath)
			{
				*OutDependencyFilePath = DependencyPaths[DepIndex];
			}
			return DepEntry;
		}
	}

	return nullptr;
}

bool FLocTextHelper::AddSourceText(const FLocKey& InNamespace, const FLocItem& InSource, const FManifestContext& InContext, const FString* InDescription)
{
	checkf(Manifest.IsValid(), TEXT("Attempted to add source text, but no manifest has been loaded!"));
	
	bool bAddSuccessful = false;

	// Check if the entry already exists in the manifest or one of the manifest dependencies
	FString ExistingEntryFileName;
	TSharedPtr<FManifestEntry> ExistingEntry = Manifest->FindEntryByContext(InNamespace, InContext);
	if (!ExistingEntry.IsValid())
	{
		ExistingEntry = FindDependencyEntry(InNamespace, InContext, &ExistingEntryFileName);

		// Ignore this dependency if the platforms are different
		if (ExistingEntry.IsValid())
		{
			const FManifestContext* DependencyContext = ExistingEntry->FindContext(InContext.Key, InContext.KeyMetadataObj);
			if (InContext.PlatformName != DependencyContext->PlatformName)
			{
				ExistingEntry.Reset();
				ExistingEntryFileName.Reset();
			}
		}
	}

	if (ExistingEntry.IsValid())
	{
		if (InSource.IsExactMatch(ExistingEntry->Source))
		{
			bAddSuccessful = true;
			ExistingEntry->MergeContextPlatformInfo(InContext);
		}
		else
		{
			// Grab the source location of the conflicting context
			const FManifestContext* ConflictingContext = ExistingEntry->FindContext(InContext.Key, InContext.KeyMetadataObj);
			const FString& ExistingEntrySourceLocation = (!ExistingEntryFileName.IsEmpty()) ? ExistingEntryFileName : ConflictingContext->SourceLocation;

			UE_LOGFMT(LogLocTextHelper, Warning, "Text conflict{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. First entry is {location}:'{text}'{textMetaData}. Conflicting entry is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.", 
				("from", (InDescription ? *FString::Printf(TEXT(" from %s"), **InDescription) : TEXT(""))),
				("location", *InContext.SourceLocation),
				("locNamespace", *SanitizeLogOutput(InNamespace.GetString())),
				("locKey", *SanitizeLogOutput(InContext.Key.GetString())),
				("locKeyMetaData", (InContext.KeyMetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(InContext.KeyMetadataObj))) : TEXT(""))),
				("text", *SanitizeLogOutput(InSource.Text)),
				("textMetaData", (InSource.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(InSource.MetadataObj))) : TEXT(""))),
				("conflictLocation", *ExistingEntrySourceLocation),
				("conflictText", *SanitizeLogOutput(ExistingEntry->Source.Text)),
				("conflictTextMetaData", (ExistingEntry->Source.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *SanitizeLogOutput(FJsonInternationalizationMetaDataSerializer::MetadataToString(ExistingEntry->Source.MetadataObj))) : TEXT(""))),
				("id", LocTextHelper::LocalizationLogIdentifier)
			);

			ConflictTracker.AddConflict(InNamespace, InContext.Key, InContext.KeyMetadataObj, InSource, InContext.SourceLocation);
			ConflictTracker.AddConflict(InNamespace, InContext.Key, InContext.KeyMetadataObj, ExistingEntry->Source, ExistingEntrySourceLocation);
		}
	}
	else
	{
		bAddSuccessful = Manifest->AddSource(InNamespace, InSource, InContext);
		if (!bAddSuccessful)
		{
			UE_LOGFMT(LogLocTextHelper, Error, "{location}: Failed to add text{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData}.",
				("location", *InContext.SourceLocation),
				("from", (InDescription ? *FString::Printf(TEXT(" from %s"), **InDescription) : TEXT(""))),
				("locNamespace", *InNamespace.GetString()),
				("locKey", *InContext.Key.GetString()),
				("locKeyMetaData", (InContext.KeyMetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(InContext.KeyMetadataObj)) : TEXT(""))),
				("text", *InSource.Text),
				("textMetaData", (InSource.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(InSource.MetadataObj)) : TEXT(""))),
				("id", LocTextHelper::LocalizationLogIdentifier)
			);
		}
	}

	return bAddSuccessful;
}

void FLocTextHelper::UpdateSourceText(const TSharedRef<FManifestEntry>& InOldEntry, TSharedRef<FManifestEntry>& InNewEntry)
{
	checkf(Manifest.IsValid(), TEXT("Attempted to update source text, but no manifest has been loaded!"));
	Manifest->UpdateEntry(InOldEntry, InNewEntry);
}

TSharedPtr<FManifestEntry> FLocTextHelper::FindSourceText(const FLocKey& InNamespace, const FLocKey& InKey, const FString* InSourceText) const
{
	checkf(Manifest.IsValid(), TEXT("Attempted to find source text, but no manifest has been loaded!"));
	return Manifest->FindEntryByKey(InNamespace, InKey, InSourceText);
}

TSharedPtr<FManifestEntry> FLocTextHelper::FindSourceText(const FLocKey& InNamespace, const FManifestContext& InContext) const
{
	checkf(Manifest.IsValid(), TEXT("Attempted to find source text, but no manifest has been loaded!"));
	return Manifest->FindEntryByContext(InNamespace, InContext);
}

void FLocTextHelper::EnumerateSourceTexts(const FEnumerateSourceTextsFuncPtr& InCallback, const bool InCheckDependencies) const
{
	checkf(Manifest.IsValid(), TEXT("Attempted to enumerate source texts, but no manifest has been loaded!"));

	for (FManifestEntryByStringContainer::TConstIterator It(Manifest->GetEntriesBySourceTextIterator()); It; ++It)
	{
		const TSharedRef<FManifestEntry> ManifestEntry = It.Value();

		bool bShouldEnumerate = true;
		if (InCheckDependencies)
		{
			for (const TSharedPtr<FInternationalizationManifest>& DepManifest : Dependencies)
			{
				const TSharedPtr<FManifestEntry> DepEntry = DepManifest->FindEntryBySource(ManifestEntry->Namespace, ManifestEntry->Source);
				if (DepEntry.IsValid())
				{
					bShouldEnumerate = false;
					break;
				}
			}
		}

		if (bShouldEnumerate && !InCallback(ManifestEntry))
		{
			break;
		}
	}
	
}

void FLocTextHelper::EnumerateSourceTextsParallel(const FEnumerateSourceTextsFuncPtr& InCallback, const bool InCheckDependencies) const
{
	checkf(Manifest.IsValid(), TEXT("Attempted to enumerate source texts, but no manifest has been loaded!"));

	std::atomic<bool> bBreak = false;

	TArray<TPair<FString, TSharedRef<FManifestEntry>>> ManifestEntries = Manifest->GetEntriesBySourceTextAsArray();
	ParallelFor(ManifestEntries.Num(), [this, &ManifestEntries, &bBreak, &InCallback, &InCheckDependencies](int32 Index)
		{
			if (bBreak.load(std::memory_order_acquire))
			{
				return;
			}

			const TSharedRef<FManifestEntry> ManifestEntry = ManifestEntries[Index].Value;

			bool bShouldEnumerate = true;
			if (InCheckDependencies)
			{
				for (const TSharedPtr<FInternationalizationManifest>& DepManifest : Dependencies)
				{
					const TSharedPtr<FManifestEntry> DepEntry = DepManifest->FindEntryBySource(ManifestEntry->Namespace, ManifestEntry->Source);
					if (DepEntry.IsValid())
					{
						bShouldEnumerate = false;
						break;
					}
				}
			}

			if (bShouldEnumerate && !InCallback(ManifestEntry))
			{
				bBreak.store(true, std::memory_order_release);
			}
		});
}

bool FLocTextHelper::AddTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation, const bool InOptional)
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to add a translation, but no valid archive could be found for '%s'!"), *InCulture);
	return Archive->AddEntry(InNamespace, InKey, InSource, InTranslation, InKeyMetadataObj, InOptional);
}

bool FLocTextHelper::AddTranslation(const FString& InCulture, const TSharedRef<FArchiveEntry>& InEntry)
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to add a translation, but no valid archive could be found for '%s'!"), *InCulture);
	return Archive->AddEntry(InEntry);
}

bool FLocTextHelper::UpdateTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation)
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to update a translation, but no valid archive could be found for '%s'!"), *InCulture);
	return Archive->SetTranslation(InNamespace, InKey, InSource, InTranslation, InKeyMetadataObj);
}

void FLocTextHelper::UpdateTranslation(const FString& InCulture, const TSharedRef<FArchiveEntry>& InOldEntry, const TSharedRef<FArchiveEntry>& InNewEntry)
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to update a translation, but no valid archive could be found for '%s'!"), *InCulture);
	Archive->UpdateEntry(InOldEntry, InNewEntry);
}

bool FLocTextHelper::ImportTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation, const bool InOptional)
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to update a translation, but no valid archive could be found for '%s'!"), *InCulture);

	// First try and update an existing entry...
	if (Archive->SetTranslation(InNamespace, InKey, InSource, InTranslation, InKeyMetadataObj))
	{
		return true;
	}

	// ... failing that, try to add a new entry
	return Archive->AddEntry(InNamespace, InKey, InSource, InTranslation, InKeyMetadataObj, InOptional);
}

TSharedPtr<FArchiveEntry> FLocTextHelper::FindTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj) const
{
	return FindTranslationImpl(InCulture, InNamespace, InKey, InKeyMetadataObj);
}

void FLocTextHelper::EnumerateTranslations(const FString& InCulture, const FEnumerateTranslationsFuncPtr& InCallback, const bool InCheckDependencies) const
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to enumerate translations, but no valid archive could be found for '%s'!"), *InCulture);

	EnumerateSourceTexts([&](TSharedRef<FManifestEntry> InManifestEntry) -> bool
	{
		bool bContinue = true;
		
		for (const FManifestContext& ManifestContext : InManifestEntry->Contexts)
		{
			TSharedPtr<FArchiveEntry> ArchiveEntry = FindTranslation(InCulture, InManifestEntry->Namespace, ManifestContext.Key, ManifestContext.KeyMetadataObj);
			if (ArchiveEntry.IsValid() && !InCallback(ArchiveEntry.ToSharedRef()))
			{
				bContinue = false;
				break;
			}
		}

		return bContinue;
	}, InCheckDependencies);
}

void FLocTextHelper::GetExportText(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const ELocTextExportSourceMethod InSourceMethod, const FLocItem& InSource, FLocItem& OutSource, FLocItem& OutTranslation) const
{
	// Default to the raw source text for the case where we're not using native translations as source
	OutSource = InSource;
	OutTranslation = FLocItem();

	if (InSourceMethod == ELocTextExportSourceMethod::NativeText && !NativeCulture.IsEmpty() && InCulture != NativeCulture)
	{
		TSharedPtr<FArchiveEntry> NativeArchiveEntry = FindTranslationImpl(NativeCulture, InNamespace, InKey, InKeyMetadataObj);
		if (NativeArchiveEntry.IsValid() && !NativeArchiveEntry->Source.IsExactMatch(NativeArchiveEntry->Translation))
		{
			// Use the native translation as the source
			OutSource = NativeArchiveEntry->Translation;
		}
	}

	TSharedPtr<FArchiveEntry> ArchiveEntry = FindTranslationImpl(InCulture, InNamespace, InKey, InKeyMetadataObj);
	if (ArchiveEntry.IsValid())
	{
		// Set the export text to use the current translation if the entry source matches the export source
		if (ArchiveEntry->Source.IsExactMatch(OutSource))
		{
			OutTranslation = ArchiveEntry->Translation;
		}
	}
	
	// We use the source text as the default translation for the native culture
	if (OutTranslation.Text.IsEmpty() && !NativeCulture.IsEmpty() && InCulture == NativeCulture)
	{
		OutTranslation = OutSource;
	}
}

void FLocTextHelper::GetRuntimeText(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const ELocTextExportSourceMethod InSourceMethod, const FLocItem& InSource, FLocItem& OutTranslation, const bool bSkipSourceCheck) const
{
	OutTranslation = InSource;

	TSharedPtr<FArchiveEntry> ArchiveEntry = FindTranslationImpl(InCulture, InNamespace, InKey, InKeyMetadataObj);
	if (ArchiveEntry.IsValid() && !ArchiveEntry->Translation.Text.IsEmpty())
	{
		if (bSkipSourceCheck)
		{
			// Set the export text to use the current translation
			OutTranslation = ArchiveEntry->Translation;
		}
		else
		{
			FLocItem ExpectedSource = InSource;

			if (InSourceMethod == ELocTextExportSourceMethod::NativeText && !NativeCulture.IsEmpty() && InCulture != NativeCulture)
			{
				TSharedPtr<FArchiveEntry> NativeArchiveEntry = FindTranslationImpl(NativeCulture, InNamespace, InKey, InKeyMetadataObj);
				if (NativeArchiveEntry.IsValid() && !NativeArchiveEntry->Source.IsExactMatch(NativeArchiveEntry->Translation))
				{
					// Use the native translation as the source
					ExpectedSource = NativeArchiveEntry->Translation;
				}
			}

			if (ArchiveEntry->Source.IsExactMatch(ExpectedSource))
			{
				// Set the export text to use the current translation
				OutTranslation = ArchiveEntry->Translation;
			}
		}
	}
}

void FLocTextHelper::AddConflict(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadata, const FLocItem& InSource, const FString& InSourceLocation)
{
	ConflictTracker.AddConflict(InNamespace, InKey, InKeyMetadata, InSource, InSourceLocation);
}

FString FLocTextHelper::GetConflictReport() const
{
	return ConflictTracker.GetConflictReport();
}

bool FLocTextHelper::SaveConflictReport(const FString& InReportFilePath, EConflictReportFormat InConflictReportFormat, FText* OutError) const
{
	FString ConflictReport;
	switch (InConflictReportFormat)
	{
		case EConflictReportFormat::Txt: 
		{
			ConflictReport = ConflictTracker.GetConflictReport();
			break;
		}
		case EConflictReportFormat::CSV:
		{
			ConflictReport = ConflictTracker.GetConflictReportAsCSV();
			break;
		}
		default:
		{
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("Error_UnsupportedConflictReportFormat", "Failed to generate conflict report '{0}'. An unsupported conflict report format was specified. Supported formats can be found in EConflictReportFormat."), FText::FromString(InReportFilePath));
			}
			return false;
		}
	}
	
	const bool bSaved = UE::Localization::FileUtil::WriteFileIfModified(InReportFilePath,
		[&ConflictReport](const FString& Filename)
		{
			return FFileHelper::SaveStringToFile(ConflictReport, *Filename);
		}, &UE::Localization::FileUtil::TextHash, LocFileNotifies);

	if (!bSaved && OutError)
	{
		*OutError = FText::Format(LOCTEXT("Error_SaveConflictReport_SaveStringToFile", "Failed to save conflict report '{0}'."), FText::FromString(InReportFilePath));
	}

	return bSaved;
}

FLocTextWordCounts FLocTextHelper::GetWordCountReport(const FDateTime& InTimestamp, const TCHAR* InBaseReportFilePath) const
{
	FLocTextWordCounts WordCounts;

	// Utility to count the number of words within a string (we use a line-break iterator to avoid counting the whitespace between the words)
	TSharedRef<IBreakIterator> LineBreakIterator = FBreakIterator::CreateLineBreakIterator();
	auto CountWords = [&LineBreakIterator](const FString& InTextToCount) -> int32
	{
		int32 NumWords = 0;
		LineBreakIterator->SetString(InTextToCount);

		int32 PreviousBreak = 0;
		int32 CurrentBreak = 0;

		while ((CurrentBreak = LineBreakIterator->MoveToNext()) != INDEX_NONE)
		{
			if (CurrentBreak > PreviousBreak)
			{
				++NumWords;
			}
			PreviousBreak = CurrentBreak;
		}

		LineBreakIterator->ClearString();
		return NumWords;
	};

	// First load in the base report
	if (InBaseReportFilePath && FPaths::FileExists(InBaseReportFilePath))
	{
		FString BaseReportCSV;
		if (FFileHelper::LoadFileToString(BaseReportCSV, InBaseReportFilePath))
		{
			FText BaseReportError;
			if (!WordCounts.FromCSV(BaseReportCSV, &BaseReportError))
			{
				UE_LOGFMT(LogLocTextHelper, Warning, "Failed to parse base word count report '{report}': {reportError}",
					("report", InBaseReportFilePath),
					("reportError", *BaseReportError.ToString()),
					("id", LocTextHelper::LocalizationLogIdentifier)
				);
			}
		}
		else
		{
			UE_LOGFMT(LogLocTextHelper, Warning, "Failed to load base word count report '{report}'.", 
				("report", InBaseReportFilePath),
				("id", LocTextHelper::LocalizationLogIdentifier)
			);
		}
	}

	// Then add our new entry (if the last entry in the report has the same timestamp as the one we were given, then replace the data in that entry rather than add a new one)
	FLocTextWordCounts::FRowData* WordCountRowData = nullptr;
	if (WordCounts.GetRowCount() > 0)
	{
		FLocTextWordCounts::FRowData* LastRowData = WordCounts.GetRow(WordCounts.GetRowCount() - 1);
		check(LastRowData);
		if (LastRowData->Timestamp == InTimestamp)
		{
			WordCountRowData = LastRowData;
			WordCountRowData->ResetWordCounts();
		}
	}
	if (!WordCountRowData)
	{
		WordCountRowData = &WordCounts.AddRow();
		WordCountRowData->Timestamp = InTimestamp;
	}

	// Count the number of source text words
	{
		TSet<FLocKey> CountedEntries;
		EnumerateSourceTexts([&WordCountRowData, &CountedEntries, &CountWords](TSharedRef<FManifestEntry> InManifestEntry) -> bool
		{
			const int32 NumWords = CountWords(InManifestEntry->Source.Text);

			// Gather relevant info from each manifest entry
			for (const FManifestContext& Context : InManifestEntry->Contexts)
			{
				if (!Context.bIsOptional)
				{
					const FLocKey CountedEntryId = FString::Printf(TEXT("%s::%s::%s"), *InManifestEntry->Source.Text, *InManifestEntry->Namespace.GetString(), *Context.Key.GetString());
					if (!CountedEntries.Contains(CountedEntryId))
					{
						WordCountRowData->SourceWordCount += NumWords;

						bool IsAlreadySet = false;
						CountedEntries.Add(CountedEntryId, &IsAlreadySet);
						check(!IsAlreadySet);
					}
				}
			}

			return true; // continue enumeration
		}, true);
	}

	// Count the number of per-culture translation words
	for (const FString& CultureName : GetAllCultures())
	{
		int32& PerCultureWordCount = WordCountRowData->PerCultureWordCounts.Add(CultureName, 0);
		TSet<FLocKey> CountedEntries;

		// Finds all the manifest entries in the archive and adds the source text word count to the running total if there is a valid translation.
		EnumerateSourceTexts([this, &CultureName, &PerCultureWordCount, &CountedEntries, &CountWords](TSharedRef<FManifestEntry> InManifestEntry) -> bool
		{
			const int32 NumWords = CountWords(InManifestEntry->Source.Text);

			// Gather relevant info from each manifest entry
			for (const FManifestContext& Context : InManifestEntry->Contexts)
			{
				if (!Context.bIsOptional)
				{
					// Use the exported text when counting as it will take native translations into account
					FLocItem WordCountSource;
					FLocItem WordCountTranslation;
					GetExportText(CultureName, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, ELocTextExportSourceMethod::NativeText, InManifestEntry->Source, WordCountSource, WordCountTranslation);

					if (!WordCountTranslation.Text.IsEmpty())
					{
						const FLocKey CountedEntryId = FString::Printf(TEXT("%s::%s::%s"), *InManifestEntry->Source.Text, *InManifestEntry->Namespace.GetString(), *Context.Key.GetString());
						if (!CountedEntries.Contains(CountedEntryId))
						{
							PerCultureWordCount += NumWords;

							bool IsAlreadySet = false;
							CountedEntries.Add(CountedEntryId, &IsAlreadySet);
							check(!IsAlreadySet);
						}
					}
				}
			}

			return true; // continue enumeration
		}, true);
	}

	return WordCounts;
}

bool FLocTextHelper::SaveWordCountReport(const FDateTime& InTimestamp, const FString& InReportFilePath, FText* OutError) const
{
	FLocTextWordCounts WordCounts = GetWordCountReport(InTimestamp, *InReportFilePath);
	WordCounts.TrimReport();

	const bool bSaved = UE::Localization::FileUtil::WriteFileIfModified(InReportFilePath,
		[&WordCounts](const FString& Filename)
		{
			return FFileHelper::SaveStringToFile(WordCounts.ToCSV(), *Filename);
		}, &UE::Localization::FileUtil::TextHash, LocFileNotifies);

	if (!bSaved && OutError)
	{
		*OutError = FText::Format(LOCTEXT("Error_SaveWordCountReport_SaveStringToFile", "Failed to save word count report '{0}'."), FText::FromString(InReportFilePath));
	}

	return bSaved;
}

FString FLocTextHelper::SanitizeLogOutput(FStringView InString)
{
	return SanitizeLogOutput(FString(InString));
}

FString FLocTextHelper::SanitizeLogOutput(FString&& ResultStr)
{
	if (ResultStr.IsEmpty())
	{
		return MoveTemp(ResultStr);
	}

	ResultStr.ReplaceCharWithEscapedCharInline();

	if (!GIsBuildMachine)
	{
		return MoveTemp(ResultStr);
	}

	static const TCHAR* ErrorStrs[] = {
		TEXT("Error"),
		TEXT("Failed"),
		TEXT("[BEROR]"),
		TEXT("Utility finished with exit code: -1"),
		TEXT("is not recognized as an internal or external command"),
		TEXT("Could not open solution: "),
		TEXT("Parameter format not correct"),
		TEXT("Another build is already started on this computer."),
		TEXT("Sorry but the link was not completed because memory was exhausted."),
		TEXT("simply rerunning the compiler might fix this problem"),
		TEXT("No connection could be made because the target machine actively refused"),
		TEXT("Internal Linker Exception:"),
		TEXT(": warning LNK4019: corrupt string table"),
		TEXT("Proxy could not update its cache"),
		TEXT("You have not agreed to the Xcode license agreements"),
		TEXT("Connection to build service terminated"),
		TEXT("cannot execute binary file"),
		TEXT("Invalid solution configuration"),
		TEXT("is from a previous version of this application and must be converted in order to build"),
		TEXT("This computer has not been authenticated for your account using Steam Guard"),
		TEXT("invalid name for SPA section"),
		TEXT(": Invalid file name, "),
		TEXT("The specified PFX file do not exist. Aborting"),
		TEXT("binary is not found. Aborting"),
		TEXT("Input file not found: "),
		TEXT("An exception occurred during merging:"),
		TEXT("Install the 'Microsoft Windows SDK for Windows 7 and .NET Framework 3.5 SP1'"),
		TEXT("is less than package's new version 0x"),
		TEXT("current engine version is older than version the package was originally saved with"),
		TEXT("exceeds maximum length"),
		TEXT("can't edit exclusive file already opened"),
	};

	static const TArray<FString> ReplacementStrings = []()
	{
		TArray<FString> TmpReplacementStrings;
		TmpReplacementStrings.Reserve(UE_ARRAY_COUNT(ErrorStrs));
		for (const TCHAR* ErrorStr : ErrorStrs)
		{
			FString ReplacementStr = TmpReplacementStrings.Add_GetRef(ErrorStr);
			ReplacementStr.InsertAt(1, TEXT(' '));
		}
		return TmpReplacementStrings;
	}();

	check(UE_ARRAY_COUNT(ErrorStrs) == ReplacementStrings.Num());

	for (int32 ErrorIndex = 0; ErrorIndex < UE_ARRAY_COUNT(ErrorStrs); ++ErrorIndex)
	{
		ResultStr.ReplaceInline(ErrorStrs[ErrorIndex], *ReplacementStrings[ErrorIndex]);
	}

	return MoveTemp(ResultStr);
}

bool FLocTextHelper::FindKeysForLegacyTranslation(const FString& InCulture, const FLocKey& InNamespace, const FString& InSource, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, TArray<FLocKey>& OutKeys) const
{
	checkf(Manifest.IsValid(), TEXT("Attempted to find a key for a legacy translation, but no manifest has been loaded!"));

	TSharedPtr<FInternationalizationArchive> NativeArchive;
	if (!NativeCulture.IsEmpty() && InCulture != NativeCulture)
	{
		NativeArchive = Archives.FindRef(NativeCulture);
		checkf(NativeArchive.IsValid(), TEXT("Attempted to find a key for a legacy translation, but no valid archive could be found for '%s'!"), *NativeCulture);
	}

	return FindKeysForLegacyTranslation(Manifest.ToSharedRef(), NativeArchive, InNamespace, InSource, InKeyMetadataObj, OutKeys);
}

bool FLocTextHelper::FindKeysForLegacyTranslation(const TSharedRef<const FInternationalizationManifest>& InManifest, const TSharedPtr<const FInternationalizationArchive>& InNativeArchive, const FLocKey& InNamespace, const FString& InSource, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, TArray<FLocKey>& OutKeys)
{
	FString RealSourceText = InSource;

	// The source text may be a native translation, so we first need to check the native archive to find the real source text that will exist in the manifest
	if (InNativeArchive.IsValid())
	{
		// We don't maintain a translation -> source mapping, so we just have to brute force it
		for (FArchiveEntryByStringContainer::TConstIterator It = InNativeArchive->GetEntriesBySourceTextIterator(); It; ++It)
		{
			const TSharedRef<FArchiveEntry>& ArchiveEntry = It.Value();
			if (ArchiveEntry->Namespace == InNamespace && ArchiveEntry->Translation.Text.Equals(InSource, ESearchCase::CaseSensitive))
			{
				if (!ArchiveEntry->KeyMetadataObj.IsValid() && !InKeyMetadataObj.IsValid())
				{
					RealSourceText = ArchiveEntry->Source.Text;
					break;
				}
				else if ((InKeyMetadataObj.IsValid() != ArchiveEntry->KeyMetadataObj.IsValid()))
				{
					// If we are in here, we know that one of the metadata entries is null, if the other contains zero entries we will still consider them equivalent.
					if ((InKeyMetadataObj.IsValid() && InKeyMetadataObj->Values.Num() == 0) || (ArchiveEntry->KeyMetadataObj.IsValid() && ArchiveEntry->KeyMetadataObj->Values.Num() == 0))
					{
						RealSourceText = ArchiveEntry->Source.Text;
						break;
					}
				}
				else if (*ArchiveEntry->KeyMetadataObj == *InKeyMetadataObj)
				{
					RealSourceText = ArchiveEntry->Source.Text;
					break;
				}
			}
		}
	}

	bool bFoundKeys = false;

	TSharedPtr<FManifestEntry> ManifestEntry = InManifest->FindEntryBySource(InNamespace, FLocItem(RealSourceText));
	if (ManifestEntry.IsValid())
	{
		for (const FManifestContext& Context : ManifestEntry->Contexts)
		{
			if (Context.KeyMetadataObj.IsValid() != InKeyMetadataObj.IsValid())
			{
				continue;
			}
			else if ((!Context.KeyMetadataObj.IsValid() && !InKeyMetadataObj.IsValid()) || (*Context.KeyMetadataObj == *InKeyMetadataObj))
			{
				OutKeys.AddUnique(Context.Key);
				bFoundKeys = true;
			}
		}
	}

	return bFoundKeys;
}

TSharedPtr<FInternationalizationManifest> FLocTextHelper::LoadManifestImpl(const FString& InManifestFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	TSharedRef<FInternationalizationManifest> LocalManifest = MakeShared<FInternationalizationManifest>();

	auto LoadSingleManifest = [this, &LocalManifest, &OutError](const FString& InManifestFilePathToLoad, const FName InPlatformName) -> bool
	{
		bool bLoaded = false;

		if (LocFileNotifies.IsValid())
		{
			LocFileNotifies->PreFileRead(InManifestFilePathToLoad);
		}

		if (FJsonInternationalizationManifestSerializer::DeserializeManifestFromFile(InManifestFilePathToLoad, LocalManifest, InPlatformName))
		{
			bLoaded = true;
		}
		else if (OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_LoadManifest_DeserializeFile", "Failed to deserialize manifest '{0}'."), FText::FromString(InManifestFilePathToLoad));
		}

		if (LocFileNotifies.IsValid())
		{
			LocFileNotifies->PostFileRead(InManifestFilePathToLoad);
		}

		return bLoaded;
	};

	// Attempt to load an existing manifest first
	if (EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Load))
	{
		const bool bExists = FPaths::FileExists(InManifestFilePath);

		if (bExists)
		{
			bool bLoadedAll = true;
			bLoadedAll &= LoadSingleManifest(InManifestFilePath, FName());
			{
				// Load all per-platform manifests too
				// We always do this, as we may have changed the split config so don't want to lose data
				const FString PlatformManifestName = FPaths::GetCleanFilename(InManifestFilePath);
				const FString PlatformLocalizationPath = FPaths::GetPath(InManifestFilePath) / FPaths::GetPlatformLocalizationFolderName();
				IFileManager::Get().IterateDirectory(*PlatformLocalizationPath, [&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
				{
					if (bIsDirectory)
					{
						const FString PlatformManifestFilePath = FilenameOrDirectory / PlatformManifestName;
						if (FPaths::FileExists(PlatformManifestFilePath))
						{
							const FString SplitPlatformName = FPaths::GetCleanFilename(FilenameOrDirectory);
							bLoadedAll &= LoadSingleManifest(PlatformManifestFilePath, *SplitPlatformName);
						}
					}
					return true;
				});
			}

			if (bLoadedAll)
			{
				return LocalManifest;
			}
			else
			{
				// Don't allow fallback to Create if the file exists but could not be loaded
				return nullptr;
			}
		}
	}

	// If we're allowed to create a manifest than we can never fail (unless file is corrupt above)
	if (EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Create))
	{
		return LocalManifest;
	}

	if (OutError)
	{
		if (!EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Load))
		{
			*OutError = LOCTEXT("Error_LoadManifest_CallerDidNotSpecifyLoadOrCreate", "Caller did not specify either Load or Create.");
		}
		else
		{
			*OutError = FText::Format(LOCTEXT("Error_LoadManifest_MissingManifest", "Manifest '{0}' does not exist."), FText::FromString(InManifestFilePath));
		}
	}
	return nullptr;
}

bool FLocTextHelper::SaveManifestImpl(const TSharedRef<const FInternationalizationManifest>& InManifest, const FString& InManifestFilePath, FText* OutError) const
{
	auto SaveSingleManifest = [this, &OutError](const TSharedRef<const FInternationalizationManifest>& InManifestToSave, const FString& InManifestFilePathToSave) -> bool
	{
		const bool bSaved = UE::Localization::FileUtil::WriteFileIfModified(InManifestFilePathToSave,
			[&InManifestToSave](const FString& Filename)
			{
				return FJsonInternationalizationManifestSerializer::SerializeManifestToFile(InManifestToSave, Filename);
			}, &UE::Localization::FileUtil::TextHash, LocFileNotifies);

		if (!bSaved && OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_SaveManifest_SerializeFile", "Failed to serialize manifest '{0}'."), FText::FromString(InManifestFilePathToSave));
		}

		return bSaved;
	};

	bool bSavedAll = true;
	if (ShouldSplitPlatformData())
	{
		const FString PlatformManifestName = FPaths::GetCleanFilename(InManifestFilePath);
		const FString PlatformLocalizationPath = FPaths::GetPath(InManifestFilePath) / FPaths::GetPlatformLocalizationFolderName();

		// Split the manifest into separate entries for each platform, as well as a platform agnostic manifest
		TSharedRef<FInternationalizationManifest> PlatformAgnosticManifest = MakeShared<FInternationalizationManifest>();
		TMap<FName, TSharedRef<FInternationalizationManifest>> PerPlatformManifests;
		{
			// Always add the split platforms so that they generate an empty manifest if there are no entries for that platform in the platform agnostic manifest
			for (const FString& SplitPlatformName : GetPlatformsToSplit())
			{
				PerPlatformManifests.Add(*SplitPlatformName, MakeShared<FInternationalizationManifest>());
			}

			// Split the manifest entries based on the platform they belonged to
			for (FManifestEntryByStringContainer::TConstIterator It(InManifest->GetEntriesBySourceTextIterator()); It; ++It)
			{
				const TSharedRef<FManifestEntry> ManifestEntry = It.Value();
				for (const FManifestContext& Context : ManifestEntry->Contexts)
				{
					TSharedPtr<FInternationalizationManifest> ManifestToUpdate = PlatformAgnosticManifest;
					if (!Context.PlatformName.IsNone())
					{
						if (TSharedRef<FInternationalizationManifest>* PerPlatformManifest = PerPlatformManifests.Find(Context.PlatformName))
						{
							ManifestToUpdate = *PerPlatformManifest;
						}
					}
					check(ManifestToUpdate.IsValid());

					if (!ManifestToUpdate->AddSource(ManifestEntry->Namespace, ManifestEntry->Source, Context))
					{
						UE_LOGFMT(LogLocTextHelper, Error, "Failed to add text for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData}{forPlatform}.",
							("locNamespace", *ManifestEntry->Namespace.GetString()),
							("locKey", *Context.Key.GetString()),
							("locKeyMetaData", (Context.KeyMetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(Context.KeyMetadataObj)) : TEXT(""))),
							("text", *ManifestEntry->Source.Text),
							("textMetaData", (ManifestEntry->Source.MetadataObj ? *FString::Printf(TEXT(" (meta-data \"%s\")"), *FJsonInternationalizationMetaDataSerializer::MetadataToString(ManifestEntry->Source.MetadataObj)) : TEXT(""))),
							("forPlatform", (Context.PlatformName.IsNone() ? TEXT("") : *FString::Printf(TEXT(" for platform \"%s\""), *Context.PlatformName.ToString()))),
							("id", LocTextHelper::LocalizationLogIdentifier)
						);
					}
				}
			}
		}

		bSavedAll &= SaveSingleManifest(PlatformAgnosticManifest, InManifestFilePath);
		for (const auto& PerPlatformManifestPair : PerPlatformManifests)
		{
			const FString PlatformManifestFilePath = PlatformLocalizationPath / PerPlatformManifestPair.Key.ToString() / PlatformManifestName;
			bSavedAll &= SaveSingleManifest(PerPlatformManifestPair.Value, PlatformManifestFilePath);
		}
	}
	else
	{
		bSavedAll &= SaveSingleManifest(InManifest, InManifestFilePath);
	}
	return bSavedAll;
}

TSharedPtr<FInternationalizationArchive> FLocTextHelper::LoadArchiveImpl(const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError)
{
	TSharedRef<FInternationalizationArchive> LocalArchive = MakeShared<FInternationalizationArchive>();

	auto LoadSingleArchive = [this, &LocalArchive, &OutError](const FString& InArchiveFilePathToLoad) -> bool
	{
		bool bLoaded = false;

		if (LocFileNotifies.IsValid())
		{
			LocFileNotifies->PreFileRead(InArchiveFilePathToLoad);
		}

		TSharedPtr<FInternationalizationArchive> NativeArchive;
		if (!NativeCulture.IsEmpty())
		{
			NativeArchive = Archives.FindRef(NativeCulture);
		}

		if (FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromFile(InArchiveFilePathToLoad, LocalArchive, Manifest, NativeArchive))
		{
			bLoaded = true;
		}
		else
		{
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("Error_LoadArchive_DeserializeFile", "Failed to deserialize archive '{0}'."), FText::FromString(InArchiveFilePathToLoad));
			}
		}

		if (LocFileNotifies.IsValid())
		{
			LocFileNotifies->PostFileRead(InArchiveFilePathToLoad);
		}

		return bLoaded;
	};

	// Attempt to load an existing archive first
	if (EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Load))
	{
		const bool bExists = FPaths::FileExists(InArchiveFilePath);

		if (bExists)
		{
			bool bLoadedAll = true;
			bLoadedAll &= LoadSingleArchive(InArchiveFilePath);
			{
				// Load all per-platform archives too
				// We always do this, as we may have changed the split config so don't want to lose data
				const FString ArchiveCultureFilePath = FPaths::GetPath(InArchiveFilePath);
				const FString PlatformArchiveName = FPaths::GetCleanFilename(InArchiveFilePath);
				const FString PlatformArchiveCulture = FPaths::GetCleanFilename(ArchiveCultureFilePath);
				const FString PlatformLocalizationPath = FPaths::GetPath(ArchiveCultureFilePath) / FPaths::GetPlatformLocalizationFolderName();
				IFileManager::Get().IterateDirectory(*PlatformLocalizationPath, [&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
				{
					if (bIsDirectory)
					{
						const FString PlatformArchiveFilePath = FilenameOrDirectory / PlatformArchiveCulture / PlatformArchiveName;
						if (FPaths::FileExists(PlatformArchiveFilePath))
						{
							bLoadedAll &= LoadSingleArchive(PlatformArchiveFilePath);
						}
					}
					return true;
				});
			}
			if (bLoadedAll)
			{
				return LocalArchive;
			}
			else
			{
				// Don't allow fallback to Create if the file exists but could not be loaded
				return nullptr;
			}
		}
	}

	// If we're allowed to create an archive than we can never fail (unless file is corrupt above)
	if (EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Create))
	{
		return LocalArchive;
	}

	if (OutError)
	{
		if (!EnumHasAnyFlags(InLoadFlags, ELocTextHelperLoadFlags::Load))
		{
			*OutError = LOCTEXT("Error_LoadManifest_CallerDidNotSpecifyLoadOrCreate", "Caller did not specify either Load or Create.");
		}
		else
		{
			*OutError = FText::Format(LOCTEXT("Error_LoadArchive_MissingArchive", "Archive '{0}' does not exist."), FText::FromString(InArchiveFilePath));
		}
	}

	return nullptr;
}

bool FLocTextHelper::SaveArchiveImpl(const TSharedRef<const FInternationalizationArchive>& InArchive, const FString& InArchiveFilePath, FText* OutError) const
{
	auto SaveSingleArchive = [this, &OutError](const TSharedRef<const FInternationalizationArchive>& InArchiveToSave, const FString& InArchiveFilePathToSave) -> bool
	{
		const bool bSaved = UE::Localization::FileUtil::WriteFileIfModified(InArchiveFilePathToSave,
			[&InArchiveToSave](const FString& Filename)
			{
				return FJsonInternationalizationArchiveSerializer::SerializeArchiveToFile(InArchiveToSave, Filename);
			}, &UE::Localization::FileUtil::TextHash, LocFileNotifies);

		if (!bSaved && OutError)
		{
			*OutError = FText::Format(LOCTEXT("Error_SaveArchive_SerializeFile", "Failed to serialize archive '{0}'."), FText::FromString(InArchiveFilePathToSave));
		}

		return bSaved;
	};

	bool bSavedAll = true;
	if (ShouldSplitPlatformData())
	{
		const FString ArchiveCultureFilePath = FPaths::GetPath(InArchiveFilePath);
		const FString PlatformArchiveName = FPaths::GetCleanFilename(InArchiveFilePath);
		const FString PlatformArchiveCulture = FPaths::GetCleanFilename(ArchiveCultureFilePath);
		const FString PlatformLocalizationPath = FPaths::GetPath(ArchiveCultureFilePath) / FPaths::GetPlatformLocalizationFolderName();

		// Split the archive into separate entries for each platform, as well as a platform agnostic archive
		TSharedRef<FInternationalizationArchive> PlatformAgnosticArchive = MakeShared<FInternationalizationArchive>();
		TMap<FName, TSharedRef<FInternationalizationArchive>> PerPlatformArchives;
		{
			// Always add the split platforms so that they generate an empty archives if there are no entries for that platform in the platform agnostic archive
			for (const FString& SplitPlatformName : GetPlatformsToSplit())
			{
				PerPlatformArchives.Add(*SplitPlatformName, MakeShared<FInternationalizationArchive>());
			}

			// Split the archive entries based on the platform they belonged to
			EnumerateSourceTexts([&InArchive, &PlatformAgnosticArchive, &PerPlatformArchives](TSharedRef<FManifestEntry> InManifestEntry) -> bool
			{
				for (const FManifestContext& Context : InManifestEntry->Contexts)
				{
					TSharedPtr<FInternationalizationArchive> ArchiveToUpdate = PlatformAgnosticArchive;
					if (!Context.PlatformName.IsNone())
					{
						if (TSharedRef<FInternationalizationArchive>* PerPlatformArchive = PerPlatformArchives.Find(Context.PlatformName))
						{
							ArchiveToUpdate = *PerPlatformArchive;
						}
					}
					check(ArchiveToUpdate.IsValid());

					// Keep any translation for the source text
					TSharedPtr<FArchiveEntry> ArchiveEntry = InArchive->FindEntryByKey(InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj);
					if (ArchiveEntry.IsValid())
					{
						ArchiveToUpdate->AddEntry(ArchiveEntry.ToSharedRef());
					}
				}

				return true; // continue enumeration
			}, true);
		}

		bSavedAll &= SaveSingleArchive(PlatformAgnosticArchive, InArchiveFilePath);
		for (const auto& PerPlatformArchivePair : PerPlatformArchives)
		{
			const FString PlatformArchiveFilePath = PlatformLocalizationPath / PerPlatformArchivePair.Key.ToString() / PlatformArchiveCulture / PlatformArchiveName;
			bSavedAll &= SaveSingleArchive(PerPlatformArchivePair.Value, PlatformArchiveFilePath);
		}
	}
	else
	{
		bSavedAll &= SaveSingleArchive(InArchive, InArchiveFilePath);
	}
	return bSavedAll;
}

TSharedPtr<FArchiveEntry> FLocTextHelper::FindTranslationImpl(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj) const
{
	TSharedPtr<FInternationalizationArchive> Archive = Archives.FindRef(InCulture);
	checkf(Archive.IsValid(), TEXT("Attempted to find a translation, but no valid archive could be found for '%s'!"), *InCulture);
	return Archive->FindEntryByKey(InNamespace, InKey, InKeyMetadataObj);
}

#undef LOCTEXT_NAMESPACE

