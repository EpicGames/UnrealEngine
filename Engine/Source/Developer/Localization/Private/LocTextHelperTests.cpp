// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/AutomationTest.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationArchive.h"
#include "LocTextHelper.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLocTextHelperTest, "System.Core.Misc.LocText Helper", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

struct FLocTextHelperTestSourceEntry
{
	FString Namespace;
	FString Key;
	FString SourceText;
};

struct FLocTextHelperTestTranslationEntry
{
	FString Namespace;
	FString Key;
	FString SourceText;
	FString TranslationText;
};

bool FLocTextHelperTest::RunTest(const FString& Parameters)
{
	const int32 NumTestEntries = 100;

	const FString TestingPath = FPaths::GameAgnosticSavedDir() / TEXT("LocTextHelperTest_") + FGuid::NewGuid().ToString();

	TArray<FString> ForeignCultures;
	ForeignCultures.Add(TEXT("fr"));
	ForeignCultures.Add(TEXT("de"));

	FLocTextHelper LocTestHelper(TestingPath, TEXT("Test.manifest"), TEXT("Test.archive"), TEXT("en"), ForeignCultures, nullptr);
	LocTestHelper.LoadAll(ELocTextHelperLoadFlags::Create);

	TArray<FLocTextHelperTestSourceEntry> TestSources;
	TestSources.Reserve(NumTestEntries);

	TArray<FLocTextHelperTestTranslationEntry> EnglishTranslations;
	EnglishTranslations.Reserve(NumTestEntries);

	TArray<FLocTextHelperTestTranslationEntry> FrenchTranslations;
	FrenchTranslations.Reserve(NumTestEntries);

	TArray<FLocTextHelperTestTranslationEntry> GermanTranslations;
	GermanTranslations.Reserve(NumTestEntries);

	// Create test data
	for (int32 Index = 0; Index < NumTestEntries; ++Index)
	{
		FLocTextHelperTestSourceEntry& TestSourceEntry = TestSources[TestSources.AddDefaulted()];
		TestSourceEntry.Namespace = TEXT("Test");
		TestSourceEntry.Key = FString::Printf(TEXT("Test_%d"), Index + 1);
		TestSourceEntry.SourceText = FString::Printf(TEXT("Source Text %d"), Index + 1);

		FLocTextHelperTestTranslationEntry& TestEnglishEntry = EnglishTranslations[EnglishTranslations.AddDefaulted()];
		TestEnglishEntry.Namespace = TestSourceEntry.Namespace;
		TestEnglishEntry.Key = TestSourceEntry.Key;
		TestEnglishEntry.SourceText = TestSourceEntry.SourceText;
		TestEnglishEntry.TranslationText = FString::Printf(TEXT("English Text %d"), Index + 1);

		FLocTextHelperTestTranslationEntry& TestFrenchEntry = FrenchTranslations[FrenchTranslations.AddDefaulted()];
		TestFrenchEntry.Namespace = TestSourceEntry.Namespace;
		TestFrenchEntry.Key = TestSourceEntry.Key;
		TestFrenchEntry.SourceText = TestEnglishEntry.TranslationText;
		TestFrenchEntry.TranslationText = FString::Printf(TEXT("French Text %d"), Index + 1);

		FLocTextHelperTestTranslationEntry& TestGermanEntry = GermanTranslations[GermanTranslations.AddDefaulted()];
		TestGermanEntry.Namespace = TestSourceEntry.Namespace;
		TestGermanEntry.Key = TestSourceEntry.Key;
		TestGermanEntry.SourceText = TestEnglishEntry.TranslationText;
		TestGermanEntry.TranslationText = FString::Printf(TEXT("German Text %d"), Index + 1);
	}

	// Populate database with test data
	for (int32 Index = 0; Index < NumTestEntries; ++Index)
	{
		{
			const FLocTextHelperTestSourceEntry& TestSource = TestSources[Index];

			FManifestContext Ctx;
			Ctx.Key = TestSource.Key;
			LocTestHelper.AddSourceText(TestSource.Namespace, FLocItem(TestSource.SourceText), Ctx);
		}

		{
			FLocTextHelperTestTranslationEntry& TestEnglishEntry = EnglishTranslations[Index];
			LocTestHelper.AddTranslation(TEXT("en"), TestEnglishEntry.Namespace, TestEnglishEntry.Key, nullptr, FLocItem(TestEnglishEntry.SourceText), FLocItem(TestEnglishEntry.TranslationText), false);
		}

		{
			FLocTextHelperTestTranslationEntry& TestFrenchEntry = FrenchTranslations[Index];
			LocTestHelper.AddTranslation(TEXT("fr"), TestFrenchEntry.Namespace, TestFrenchEntry.Key, nullptr, FLocItem(TestFrenchEntry.SourceText), FLocItem(TestFrenchEntry.TranslationText), false);
		}

		{
			FLocTextHelperTestTranslationEntry& TestGermanEntry = GermanTranslations[Index];
			LocTestHelper.AddTranslation(TEXT("de"), TestGermanEntry.Namespace, TestGermanEntry.Key, nullptr, FLocItem(TestGermanEntry.SourceText), FLocItem(TestGermanEntry.TranslationText), false);
		}
	}

	// Check that all the test data can be found
	for (int32 Index = 0; Index < NumTestEntries; ++Index)
	{
		const FLocTextHelperTestSourceEntry& TestSourceEntry = TestSources[Index];
		const FLocTextHelperTestTranslationEntry& TestEnglishEntry = EnglishTranslations[Index];
		const FLocTextHelperTestTranslationEntry& TestFrenchEntry = FrenchTranslations[Index];
		const FLocTextHelperTestTranslationEntry& TestGermanEntry = GermanTranslations[Index];

		{
			TSharedPtr<FManifestEntry> ManifestEntry = LocTestHelper.FindSourceText(TestSourceEntry.Namespace, TestSourceEntry.Key, &TestSourceEntry.SourceText);
			if (!ManifestEntry.IsValid())
			{
				AddError(FString::Printf(TEXT("Failed to find expected source text: '%s', '%s', '%s'."), *TestSourceEntry.Namespace, *TestSourceEntry.Key, *TestSourceEntry.SourceText));
			}
		}

		auto TestTranslationLookup = [this, &LocTestHelper](const FString& InCultureName, const FLocTextHelperTestSourceEntry& InSourceEntry, const FLocTextHelperTestTranslationEntry& InTranslationEntry)
		{
			TSharedPtr<FArchiveEntry> FoundArchiveEntry = LocTestHelper.FindTranslation(InCultureName, InTranslationEntry.Namespace, InTranslationEntry.Key, nullptr);
			if (!FoundArchiveEntry.IsValid() || !FoundArchiveEntry->Translation.Text.Equals(InTranslationEntry.TranslationText, ESearchCase::CaseSensitive))
			{
				AddError(FString::Printf(TEXT("Failed to find expected translation for '%s': '%s', '%s', '%s', '%s'."), *InCultureName, *InTranslationEntry.Namespace, *InTranslationEntry.Key, *InTranslationEntry.SourceText, *InTranslationEntry.TranslationText));
			}

			FoundArchiveEntry = LocTestHelper.FindTranslation(InCultureName, InSourceEntry.Namespace, InSourceEntry.Key, nullptr);
			if (!FoundArchiveEntry.IsValid() || !FoundArchiveEntry->Translation.Text.Equals(InTranslationEntry.TranslationText, ESearchCase::CaseSensitive))
			{
				AddError(FString::Printf(TEXT("Failed to find expected translation for '%s': '%s', '%s', '%s', '%s'."), *InCultureName, *InSourceEntry.Namespace, *InSourceEntry.Key, *InSourceEntry.SourceText, *InTranslationEntry.TranslationText));
			}

			FLocItem FoundSourceText;
			FLocItem FoundTranslationText;
			LocTestHelper.GetExportText(InCultureName, InTranslationEntry.Namespace, InTranslationEntry.Key, nullptr, ELocTextExportSourceMethod::NativeText, FLocItem(InTranslationEntry.SourceText), FoundSourceText, FoundTranslationText);
			if (!FoundTranslationText.Text.Equals(InTranslationEntry.TranslationText, ESearchCase::CaseSensitive) && !FoundSourceText.Text.Equals(InTranslationEntry.SourceText, ESearchCase::CaseSensitive))
			{
				AddError(FString::Printf(TEXT("Failed to find expected export text for '%s': '%s', '%s', '%s', '%s'."), *InCultureName, *InTranslationEntry.Namespace, *InTranslationEntry.Key, *InTranslationEntry.SourceText, *InTranslationEntry.TranslationText));
			}

			LocTestHelper.GetExportText(InCultureName, InSourceEntry.Namespace, InSourceEntry.Key, nullptr, ELocTextExportSourceMethod::NativeText, FLocItem(InSourceEntry.SourceText), FoundSourceText, FoundTranslationText);
			if (!FoundTranslationText.Text.Equals(InTranslationEntry.TranslationText, ESearchCase::CaseSensitive) && !FoundSourceText.Text.Equals(InSourceEntry.SourceText, ESearchCase::CaseSensitive))
			{
				AddError(FString::Printf(TEXT("Failed to find expected export text for '%s': '%s', '%s', '%s', '%s'."), *InCultureName, *InTranslationEntry.Namespace, *InTranslationEntry.Key, *InTranslationEntry.SourceText, *InTranslationEntry.TranslationText));
			}
		};

		TestTranslationLookup(TEXT("en"), TestSourceEntry, TestEnglishEntry);
		TestTranslationLookup(TEXT("fr"), TestSourceEntry, TestFrenchEntry);
		TestTranslationLookup(TEXT("de"), TestSourceEntry, TestGermanEntry);
	}
	
	// Check that all the test data can be enumerated
	{
		int32 EnumeratedCount = 0;
		LocTestHelper.EnumerateSourceTexts([&EnumeratedCount](TSharedRef<FManifestEntry> InManifestEntry) -> bool
		{
			++EnumeratedCount;
			return true;
		}, true);

		if (NumTestEntries != EnumeratedCount)
		{
			AddError(FString::Printf(TEXT("Failed to enumerate the expected number of source texts. Expected: %d. Actual: %d."), NumTestEntries, EnumeratedCount));
		}
	}

	auto TestTranslationEnumeration = [this, &LocTestHelper, &NumTestEntries](const FString& InCultureName)
	{
		int32 EnumeratedCount = 0;
		LocTestHelper.EnumerateTranslations(InCultureName, [&EnumeratedCount](TSharedRef<FArchiveEntry> InArchiveEntry) -> bool
		{
			++EnumeratedCount;
			return true;
		}, true);

		if (NumTestEntries != EnumeratedCount) //-V547
		{
			AddError(FString::Printf(TEXT("Failed to enumerate the expected number of translations for '%s'. Expected: %d. Actual: %d."), *InCultureName, NumTestEntries, EnumeratedCount));
		}
	};

	TestTranslationEnumeration(TEXT("en"));
	TestTranslationEnumeration(TEXT("fr"));
	TestTranslationEnumeration(TEXT("de"));

	IFileManager::Get().DeleteDirectory(*TestingPath, true, true);

	return true;
}
