// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationConfigurationScript.h"
#include "LocalizationTargetTypes.h"
#include "LocalizationSettings.h"
#include "UObject/Package.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"

namespace LocalizationConfigSCC
{

void PreWriteFile(const FString& InFilename)
{
	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InFilename);

	if (!FPaths::FileExists(AbsoluteFilename))
	{
		return;
	}

	// Check out it if it's under SCC
	if (FLocalizationSourceControlSettings::IsSourceControlAvailable() && FLocalizationSourceControlSettings::IsSourceControlEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid() && SourceControlState->IsDeleted())
		{
			// If it's deleted, we need to revert that first
			SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), AbsoluteFilename);
			SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
		}

		if (SourceControlState.IsValid())
		{
			if (SourceControlState->IsAdded() || SourceControlState->IsCheckedOut())
			{
				// Nothing to do
			}
			else if (SourceControlState->CanCheckout())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), AbsoluteFilename);
			}
		}
	}

	// Failing that, just make it writable
	if (IFileManager::Get().IsReadOnly(*AbsoluteFilename))
	{
		FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*AbsoluteFilename, false);
	}
}

void PostWriteFile(const FString& InFilename)
{
	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InFilename);

	if (!FPaths::FileExists(AbsoluteFilename))
	{
		return;
	}

	// Add the file if it's not already under SCC
	if (FLocalizationSourceControlSettings::IsSourceControlAvailable() && FLocalizationSourceControlSettings::IsSourceControlEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid())
		{
			if (!SourceControlState->IsSourceControlled() && SourceControlState->CanAdd())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), AbsoluteFilename);
			}
		}
	}
}

}

namespace
{
	FString GetConfigDir(const ULocalizationTarget* const Target)
	{
		return Target->IsMemberOfEngineTargetSet() ? FPaths::EngineConfigDir() : FPaths::ProjectConfigDir();
	}

	FString GetContentDir(const ULocalizationTarget* const Target)
	{
		return Target->IsMemberOfEngineTargetSet() ? FPaths::EngineContentDir() : FPaths::ProjectContentDir();
	}
}

bool FLocalizationConfigurationScript::WriteWithSCC(const FString& InConfigFilename)
{
	return LocalizationConfigurationScript::WriteConfigFileWithSCC(InConfigFilename, *this);
}

namespace LocalizationConfigurationScript
{
	FString MakePathRelativeForCommandletProcess(const FString& Path, const bool IsUsingProjectFile)
	{
		FString Result = Path;
		const FString ProjectDir = !IsUsingProjectFile ? FPaths::EngineDir() : FPaths::ProjectDir();
		if (!FPaths::MakePathRelativeTo(Result, *ProjectDir))
		{
			Result = FPaths::ConvertRelativePathToFull(Path);
		}
		return Result;
	}

	FString GetConfigDirectory(const ULocalizationTarget* const Target)
	{
		return GetConfigDir(Target) / TEXT("Localization");
	}

	FString GetDataDirectory(const ULocalizationTarget* const Target)
	{
		return GetContentDir(Target) / TEXT("Localization") / Target->Settings.Name;
	}

	TArray<FString> GetConfigPaths(const ULocalizationTarget* const Target)
	{
		TArray<FString> Result;
		Result.Add(GetGatherTextConfigPath(Target));
		Result.Add(GetImportTextConfigPath(Target));
		Result.Add(GetExportTextConfigPath(Target));
		Result.Add(GetImportDialogueScriptConfigPath(Target));
		Result.Add(GetExportDialogueScriptConfigPath(Target));
		Result.Add(GetImportDialogueConfigPath(Target));
		Result.Add(GetCompileTextConfigPath(Target));
		Result.Add(GetWordCountReportConfigPath(Target));
		return Result;
	}

	void GenerateAllConfigFiles(const ULocalizationTarget* const Target)
	{
		GenerateGatherTextConfigFile(Target).WriteWithSCC(GetGatherTextConfigPath(Target));
		GenerateImportTextConfigFile(Target).WriteWithSCC(GetImportTextConfigPath(Target));
		GenerateExportTextConfigFile(Target).WriteWithSCC(GetExportTextConfigPath(Target));
		GenerateImportDialogueScriptConfigFile(Target).WriteWithSCC(GetImportDialogueScriptConfigPath(Target));
		GenerateExportDialogueScriptConfigFile(Target).WriteWithSCC(GetExportDialogueScriptConfigPath(Target));
		GenerateImportDialogueConfigFile(Target).WriteWithSCC(GetImportDialogueConfigPath(Target));
		GenerateCompileTextConfigFile(Target).WriteWithSCC(GetCompileTextConfigPath(Target));
		GenerateWordCountReportConfigFile(Target).WriteWithSCC(GetWordCountReportConfigPath(Target));
	}

	TArray<FString> GetOutputFilePaths(const ULocalizationTarget* const Target)
	{
		TArray<FString> Result;

		// Culture agnostic paths
		Result.Add(GetLocMetaPath(Target));
		Result.Add(GetManifestPath(Target));
		Result.Add(GetWordCountCSVPath(Target));
		Result.Add(GetConflictReportPath(Target));
		Result.Add(GetDataDirectory(Target));

		// Culture specific paths
		for (const FCultureStatistics& Culture : Target->Settings.SupportedCulturesStatistics)
		{
			Result.Add(GetArchivePath(Target, Culture.CultureName));
			Result.Add(GetDefaultPOPath(Target, Culture.CultureName));
			Result.Add(GetDefaultDialogueScriptPath(Target, Culture.CultureName));
			Result.Add(GetLocResPath(Target, Culture.CultureName));
		}

		return Result;
	}

	FString GetManifestFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.manifest"), *Target->Settings.Name);
	}

	FString GetManifestPath(const ULocalizationTarget* const Target)
	{
		return GetDataDirectory(Target) / GetManifestFileName(Target);
	}

	FString GetArchiveFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.archive"), *Target->Settings.Name);
	}

	FString GetArchivePath(const ULocalizationTarget* const Target, const FString& CultureName)
	{
		return GetDataDirectory(Target) / CultureName / GetArchiveFileName(Target);
	}

	FString GetDefaultPOFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.po"), *Target->Settings.Name);
	}

	FString GetDefaultPOPath(const ULocalizationTarget* const Target, const FString& CultureName)
	{
		return GetDataDirectory(Target) / CultureName / GetDefaultPOFileName(Target);
	}

	FString GetDefaultDialogueScriptFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%sDialogue.csv"), *Target->Settings.Name);
	}

	FString GetDefaultDialogueScriptPath(const ULocalizationTarget* const Target, const FString& CultureName)
	{
		return GetDataDirectory(Target) / CultureName / GetDefaultDialogueScriptFileName(Target);
	}

	FString GetLocResFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.locres"), *Target->Settings.Name);
	}

	FString GetLocResPath(const ULocalizationTarget* const Target, const FString& CultureName)
	{
		return GetDataDirectory(Target) / CultureName / GetLocResFileName(Target);
	}

	FString GetLocMetaFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.locmeta"), *Target->Settings.Name);
	}

	FString GetLocMetaPath(const ULocalizationTarget* const Target)
	{
		return GetDataDirectory(Target) / GetLocMetaFileName(Target);
	}

	FString GetWordCountCSVFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s.csv"), *Target->Settings.Name);
	}

	FString GetWordCountCSVPath(const ULocalizationTarget* const Target)
	{
		return GetDataDirectory(Target) / GetWordCountCSVFileName(Target);
	}

	FString GetConflictReportFileName(const ULocalizationTarget* const Target)
	{
		return FString::Printf(TEXT("%s_Conflicts.txt"), *Target->Settings.Name);
	}

	FString GetConflictReportPath(const ULocalizationTarget* const Target)
	{
		return GetDataDirectory(Target) / GetConflictReportFileName(Target);
	}

	FLocalizationConfigurationScript GenerateGatherTextConfigFile(const ULocalizationTarget* const Target)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ConfigDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetConfigDir(Target), !bIsEngineTarget);
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			TArray<ULocalizationTarget*> AllLocalizationTargets;

			ULocalizationTargetSet* EngineTargetSet = ULocalizationSettings::GetEngineTargetSet();
			if (EngineTargetSet)
			{
				AllLocalizationTargets.Append(EngineTargetSet->TargetObjects);
			}

			// Engine targets may not depend on game targets
			if (!bIsEngineTarget)
			{
				ULocalizationTargetSet* GameTargetSet = ULocalizationSettings::GetGameTargetSet();
				if (GameTargetSet)
				{
					AllLocalizationTargets.Append(GameTargetSet->TargetObjects);
				}
			}

			const ULocalizationTargetSet* const LocalizationTargetSet = GetDefault<ULocalizationTargetSet>(ULocalizationTargetSet::StaticClass());
			for (const FGuid& TargetDependencyGuid : Target->Settings.TargetDependencies)
			{
				const ULocalizationTarget* const * OtherTarget = AllLocalizationTargets.FindByPredicate([&TargetDependencyGuid](ULocalizationTarget* const InOtherTarget)->bool{return InOtherTarget->Settings.Guid == TargetDependencyGuid;});
				if (OtherTarget && Target != *OtherTarget)
				{
					ConfigSection.Add( TEXT("ManifestDependencies"), MakePathRelativeForCommandletProcess(GetManifestPath(*OtherTarget), !bIsEngineTarget) );
				}
			}

			for (const FFilePath& Path : Target->Settings.AdditionalManifestDependencies)
			{
				ConfigSection.Add( TEXT("ManifestDependencies"), MakePathRelativeForCommandletProcess(Path.FilePath, !bIsEngineTarget) );
			}

			for (const FString& ModuleName : Target->Settings.RequiredModuleNames)
			{
				ConfigSection.Add( TEXT("ModulesToPreload"), ModuleName );
			}

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("SourcePath"), SourcePath );
			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("DestinationPath"), DestinationPath );

			ConfigSection.Add( TEXT("ManifestName"), GetManifestFileName(Target) );
			ConfigSection.Add( TEXT("ArchiveName"), GetArchiveFileName(Target) );

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add( TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName );
			}
			for (const FCultureStatistics& CultureStatistics : Target->Settings.SupportedCulturesStatistics)
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), CultureStatistics.CultureName );
			}
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		uint32 GatherTextStepIndex = 0;
		// GatherTextFromSource
		if (Target->Settings.GatherFromTextFiles.IsEnabled)
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GatherTextFromSource") );

			// Include Paths
			for (const auto& IncludePath : Target->Settings.GatherFromTextFiles.SearchDirectories)
			{
				ConfigSection.Add( TEXT("SearchDirectoryPaths"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(IncludePath.PathRoot), *IncludePath.Path) );
			}

			// Exclude Paths
			ConfigSection.Add( TEXT("ExcludePathFilters"), TEXT("Config/Localization/*") );
			for (const auto& ExcludePath : Target->Settings.GatherFromTextFiles.ExcludePathWildcards)
			{
				ConfigSection.Add( TEXT("ExcludePathFilters"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(ExcludePath.PathRoot), *ExcludePath.Pattern) );
			}

			// Source File Search Filters
			for (const auto& FileExtension : Target->Settings.GatherFromTextFiles.FileExtensions)
			{
				ConfigSection.Add( TEXT("FileNameFilters"), FString::Printf( TEXT("*.%s"), *FileExtension.Pattern) );
			}

			ConfigSection.Add( TEXT("ShouldGatherFromEditorOnlyData"), Target->Settings.GatherFromTextFiles.ShouldGatherFromEditorOnlyData ? TEXT("true") : TEXT("false") );
			
			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		// GatherTextFromAssets
		if (Target->Settings.GatherFromPackages.IsEnabled)
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GatherTextFromAssets") );

			// Include Paths
			for (const auto& IncludePath : Target->Settings.GatherFromPackages.IncludePathWildcards)
			{
				ConfigSection.Add( TEXT("IncludePathFilters"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(IncludePath.PathRoot), *IncludePath.Pattern) );
			}

			// Exclude Paths
			ConfigSection.Add( TEXT("ExcludePathFilters"), TEXT("Content/Localization/*") );
			for (const auto& ExcludePath : Target->Settings.GatherFromPackages.ExcludePathWildcards)
			{
				ConfigSection.Add( TEXT("ExcludePathFilters"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(ExcludePath.PathRoot), *ExcludePath.Pattern) );
			}

			// Package Extensions
			for (const auto& FileExtension : Target->Settings.GatherFromPackages.FileExtensions)
			{
				ConfigSection.Add( TEXT("PackageFileNameFilters"), FString::Printf( TEXT("*.%s"), *FileExtension.Pattern) );
			}

			for (const auto& CollectionName : Target->Settings.GatherFromPackages.Collections)
			{
				ConfigSection.Add( TEXT("CollectionFilters"), CollectionName.ToString() );
			}

			for (const auto& CollectionName : Target->Settings.GatherFromPackages.WorldCollections)
			{
				ConfigSection.Add( TEXT("WorldCollectionFilters"), CollectionName.ToString() );
			}

			// Class filters
			for (const FSoftClassPath& ExcludeClass : Target->Settings.GatherFromPackages.ExcludeClasses)
			{
				if (ExcludeClass.IsValid())
				{
					ConfigSection.Add(TEXT("ExcludeClasses"), ExcludeClass.GetAssetPathString());
				}
			}
			ConfigSection.Add(TEXT("ShouldExcludeDerivedClasses"), Target->Settings.GatherFromPackages.ShouldExcludeDerivedClasses ? TEXT("true") : TEXT("false"));

			// Key Specifications
			for (const FMetaDataKeyGatherSpecification& Specification : Target->Settings.GatherFromPackages.MetadataKeySpecifications)
			{
				ConfigSection.Add(TEXT("MetadataInputKeys"), Specification.MetaDataKey.Name);
				ConfigSection.Add(TEXT("MetadataOutputNamespaces"), Specification.TextNamespace);
				ConfigSection.Add(TEXT("MetadataOutputKeys"), Specification.TextKeyPattern.Pattern);
			}

			// Field Type Filters
			for (const FString& FieldTypeToInclude : Target->Settings.GatherFromPackages.MetadataFieldTypesToInclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldTypesToInclude"), FieldTypeToInclude);
			}
			for (const FString& FieldTypeToExclude : Target->Settings.GatherFromPackages.MetadataFieldTypesToExclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldTypesToExclude"), FieldTypeToExclude);
			}

			// Field Owner Type Filters
			for (const FString& FieldOwnerTypeToInclude : Target->Settings.GatherFromPackages.MetadataFieldOwnerTypesToInclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldOwnerTypesToInclude"), FieldOwnerTypeToInclude);
			}
			for (const FString& FieldOwnerTypeToExclude : Target->Settings.GatherFromPackages.MetadataFieldOwnerTypesToExclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldOwnerTypesToExclude"), FieldOwnerTypeToExclude);
			}

			// Field Outer Type Filters
			for (const FString& FieldOuterTypeToInclude : Target->Settings.GatherFromPackages.MetadataFieldOuterTypesToInclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldOuterTypesToInclude"), FieldOuterTypeToInclude);
			}
			for (const FString& FieldOuterTypeToExclude : Target->Settings.GatherFromPackages.MetadataFieldOuterTypesToExclude)
			{
				ConfigSection.Add(TEXT("MetadataFieldOuterTypesToExclude"), FieldOuterTypeToExclude);
			}

			ConfigSection.Add( TEXT("ShouldGatherFromEditorOnlyData"), Target->Settings.GatherFromPackages.ShouldGatherFromEditorOnlyData ? TEXT("true") : TEXT("false") );
			ConfigSection.Add( TEXT("SkipGatherCache"), Target->Settings.GatherFromPackages.SkipGatherCache ? TEXT("true") : TEXT("false") );

			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		// GatherTextFromMetadata
		if (Target->Settings.GatherFromMetaData.IsEnabled)
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GatherTextFromMetadata") );

			// Include Paths
			for (const auto& IncludePath : Target->Settings.GatherFromMetaData.IncludePathWildcards)
			{
				ConfigSection.Add( TEXT("IncludePathFilters"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(IncludePath.PathRoot), *IncludePath.Pattern) );
			}

			// Exclude Paths
			for (const auto& ExcludePath : Target->Settings.GatherFromMetaData.ExcludePathWildcards)
			{
				ConfigSection.Add( TEXT("ExcludePathFilters"), FString::Printf(TEXT("%s%s"), *FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(ExcludePath.PathRoot), *ExcludePath.Pattern) );
			}

			// Key Specifications
			for (const FMetaDataKeyGatherSpecification& Specification : Target->Settings.GatherFromMetaData.KeySpecifications)
			{
				ConfigSection.Add( TEXT("InputKeys"), Specification.MetaDataKey.Name );
				ConfigSection.Add( TEXT("OutputNamespaces"), Specification.TextNamespace );
				ConfigSection.Add( TEXT("OutputKeys"), Specification.TextKeyPattern.Pattern );
			}

			// Field Type Filters
			for (const FString& FieldTypeToInclude : Target->Settings.GatherFromMetaData.FieldTypesToInclude)
			{
				ConfigSection.Add(TEXT("FieldTypesToInclude"), FieldTypeToInclude);
			}
			for (const FString& FieldTypeToExclude : Target->Settings.GatherFromMetaData.FieldTypesToExclude)
			{
				ConfigSection.Add(TEXT("FieldTypesToExclude"), FieldTypeToExclude);
			}

			// Field Owner Type Filters
			for (const FString& FieldOwnerTypeToInclude : Target->Settings.GatherFromMetaData.FieldOwnerTypesToInclude)
			{
				ConfigSection.Add(TEXT("FieldOwnerTypesToInclude"), FieldOwnerTypeToInclude);
			}
			for (const FString& FieldOwnerTypeToExclude : Target->Settings.GatherFromMetaData.FieldOwnerTypesToExclude)
			{
				ConfigSection.Add(TEXT("FieldOwnerTypesToExclude"), FieldOwnerTypeToExclude);
			}

			// Field Outer Type Filters
			for (const FString& FieldOuterTypeToInclude : Target->Settings.GatherFromMetaData.FieldOuterTypesToInclude)
			{
				ConfigSection.Add(TEXT("FieldOuterTypesToInclude"), FieldOuterTypeToInclude);
			}
			for (const FString& FieldOuterTypeToExclude : Target->Settings.GatherFromMetaData.FieldOuterTypesToExclude)
			{
				ConfigSection.Add(TEXT("FieldOuterTypesToExclude"), FieldOuterTypeToExclude);
			}

			ConfigSection.Add( TEXT("ShouldGatherFromEditorOnlyData"), Target->Settings.GatherFromMetaData.ShouldGatherFromEditorOnlyData ? TEXT("true") : TEXT("false") );

			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		// GenerateGatherManifest
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GenerateGatherManifest") );

			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		// GenerateGatherArchive
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GenerateGatherArchive") );

			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		// GenerateTextLocalizationReport
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GenerateTextLocalizationReport") );

			ConfigSection.Add( TEXT("bWordCountReport"), TEXT("true") );
			ConfigSection.Add( TEXT("WordCountReportName"), GetWordCountCSVFileName(Target) );

			ConfigSection.Add( TEXT("bConflictReport"), TEXT("true") );
			ConfigSection.Add( TEXT("ConflictReportName"), GetConflictReportFileName(Target) );

			Script.AddGatherTextStep(GatherTextStepIndex++, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetGatherTextConfigPath(const ULocalizationTarget* const Target)
	{
		return GetConfigDirectory(Target) / FString::Printf(TEXT("%s_Gather.ini"), *(Target->Settings.Name));
	}

	FLocalizationConfigurationScript GenerateImportTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName, const TOptional<FString> ImportPathOverride)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			FString SourcePath;
			// Overriding output path changes the source directory for the PO file.
			if (ImportPathOverride.IsSet())
			{
				// The output path for a specific culture is a file path.
				if (CultureName.IsSet())
				{
					SourcePath = MakePathRelativeForCommandletProcess( FPaths::GetPath(ImportPathOverride.GetValue()), !bIsEngineTarget );
				}
				// Otherwise, it is a directory path.
				else
				{
					SourcePath = MakePathRelativeForCommandletProcess( ImportPathOverride.GetValue(), !bIsEngineTarget );
				}
			}
			// Use the default PO file's directory path.
			else
			{
				SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			}
			ConfigSection.Add( TEXT("SourcePath"), SourcePath );

			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("DestinationPath"), DestinationPath );

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add( TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName );
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName );
			};

			// Import for a specific culture.
			if (CultureName.IsSet())
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), CultureName.GetValue() );
			}
			// Import for all cultures.
			else
			{
				for (const FCultureStatistics& CultureStatistics : Target->Settings.SupportedCulturesStatistics)
				{
					ConfigSection.Add( TEXT("CulturesToGenerate"), CultureStatistics.CultureName );
				}
			}

			// Do not use culture subdirectories if importing a single culture from a specific directory.
			if (CultureName.IsSet() && ImportPathOverride.IsSet())
			{
				ConfigSection.Add( TEXT("bUseCultureDirectory"), TEXT("false") );
			}

			ConfigSection.Add( TEXT("ManifestName"), GetManifestFileName(Target) );
			ConfigSection.Add( TEXT("ArchiveName"), GetArchiveFileName(Target) );

			FString POFileName;
			// The import path for a specific culture is a file path.
			if (CultureName.IsSet() && ImportPathOverride.IsSet())
			{
				POFileName =  FPaths::GetCleanFilename( ImportPathOverride.GetValue() );
			}
			// Use the default PO file's name.
			else
			{
				POFileName = GetDefaultPOFileName( Target );
			}
			ConfigSection.Add( TEXT("PortableObjectName"), POFileName );
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - InternationalizationExport
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("InternationalizationExport") );

			ConfigSection.Add( TEXT("bImportLoc"), TEXT("true") );

			// Import-specific settings.
			{
				UEnum* LocalizedTextCollapseModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizedTextCollapseMode"));
				const FName CollapseModeName = LocalizedTextCollapseModeEnum->GetNameByValue((int64)Target->Settings.ExportSettings.CollapseMode);
				ConfigSection.Add(TEXT("LocalizedTextCollapseMode"), CollapseModeName.ToString());

				UEnum* POFormatEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.EPortableObjectFormat"));
				const FName POFormatName = POFormatEnum->GetNameByValue((int64)Target->Settings.ExportSettings.POFormat);
				ConfigSection.Add(TEXT("POFormat"), POFormatName.ToString());
			}
			
			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetImportTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Import_%s.ini"), *Target->Settings.Name, *CultureName.GetValue() );
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Import.ini"), *Target->Settings.Name );
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateExportTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName, const TOptional<FString> ExportPathOverride)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("SourcePath"), SourcePath );

			FString DestinationPath;
			// Overriding export path changes the destination directory for the PO file.
			if (ExportPathOverride.IsSet())
			{
				// The output path for a specific culture is a file path.
				if (CultureName.IsSet())
				{
					DestinationPath = MakePathRelativeForCommandletProcess( FPaths::GetPath(ExportPathOverride.GetValue()), !bIsEngineTarget );
				}
				// Otherwise, it is a directory path.
				else
				{
					DestinationPath = MakePathRelativeForCommandletProcess( ExportPathOverride.GetValue(), !bIsEngineTarget );
				}
			}
			// Use the default PO file's directory path.
			else
			{
				DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			}
			ConfigSection.Add( TEXT("DestinationPath"), DestinationPath );

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add( TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName );
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName );
			};

			// Export for a specific culture.
			if (CultureName.IsSet())
			{
				const int32 CultureIndex = Target->Settings.SupportedCulturesStatistics.IndexOfByPredicate([CultureName](const FCultureStatistics& Culture) { return Culture.CultureName == CultureName.GetValue(); });
				AddCultureToGenerate(CultureIndex);
			}
			// Export for all cultures.
			else
			{
				for (int32 CultureIndex = 0; CultureIndex < Target->Settings.SupportedCulturesStatistics.Num(); ++CultureIndex)
				{
					AddCultureToGenerate(CultureIndex);
				}
			}

			// Do not use culture subdirectories if exporting a single culture to a specific directory.
			if (CultureName.IsSet() && ExportPathOverride.IsSet())
			{
				ConfigSection.Add( TEXT("bUseCultureDirectory"), TEXT("false") );
			}

			ConfigSection.Add( TEXT("ManifestName"), GetManifestFileName(Target) );
			ConfigSection.Add( TEXT("ArchiveName"), GetArchiveFileName(Target) );

			FString POFileName;
			// The export path for a specific culture is a file path.
			if (CultureName.IsSet() && ExportPathOverride.IsSet())
			{
				POFileName =  FPaths::GetCleanFilename( ExportPathOverride.GetValue() );
			}
			// Use the default PO file's name.
			else
			{
				POFileName = GetDefaultPOFileName(Target);
			}
			ConfigSection.Add( TEXT("PortableObjectName"), POFileName );
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - InternationalizationExport
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("InternationalizationExport") );

			ConfigSection.Add(TEXT("bExportLoc"), TEXT("true"));

			// Export-specific settings.
			{
				UEnum* LocalizedTextCollapseModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizedTextCollapseMode"));
				const FName CollapseModeName = LocalizedTextCollapseModeEnum->GetNameByValue((int64)Target->Settings.ExportSettings.CollapseMode);
				ConfigSection.Add(TEXT("LocalizedTextCollapseMode"), CollapseModeName.ToString());

				UEnum* POFormatEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.EPortableObjectFormat"));
				const FName POFormatName = POFormatEnum->GetNameByValue((int64)Target->Settings.ExportSettings.POFormat);
				ConfigSection.Add(TEXT("POFormat"), POFormatName.ToString());

				ConfigSection.Add(TEXT("ShouldPersistCommentsOnExport"), Target->Settings.ExportSettings.ShouldPersistCommentsOnExport ? TEXT("true") : TEXT("false"));
				ConfigSection.Add(TEXT("ShouldAddSourceLocationsAsComments"), Target->Settings.ExportSettings.ShouldAddSourceLocationsAsComments ? TEXT("true") : TEXT("false"));
			}
			
			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetExportTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Export_%s.ini"), *Target->Settings.Name, *CultureName.GetValue() );
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Export.ini"), *Target->Settings.Name );
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateImportDialogueScriptConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName, const TOptional<FString> ImportPathOverride)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			FString SourcePath;
			// Overriding import path changes the source directory for the dialogue script file.
			if (ImportPathOverride.IsSet())
			{
				// The output path for a specific culture is a file path.
				if (CultureName.IsSet())
				{
					SourcePath = MakePathRelativeForCommandletProcess(FPaths::GetPath(ImportPathOverride.GetValue()), !bIsEngineTarget);
				}
				// Otherwise, it is a directory path.
				else
				{
					SourcePath = MakePathRelativeForCommandletProcess(ImportPathOverride.GetValue(), !bIsEngineTarget);
				}
			}
			// Use the default dialogue script file's directory path.
			else
			{
				SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			}
			ConfigSection.Add(TEXT("SourcePath"), SourcePath);

			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add(TEXT("DestinationPath"), DestinationPath);

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add(TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName);
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add(TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName);
			};

			// Import for a specific culture.
			if (CultureName.IsSet())
			{
				ConfigSection.Add(TEXT("CulturesToGenerate"), CultureName.GetValue());
			}
			// Import for all cultures.
			else
			{
				for (const FCultureStatistics& CultureStatistics : Target->Settings.SupportedCulturesStatistics)
				{
					ConfigSection.Add(TEXT("CulturesToGenerate"), CultureStatistics.CultureName);
				}
			}

			// Do not use culture subdirectories if importing a single culture from a specific directory.
			if (CultureName.IsSet() && ImportPathOverride.IsSet())
			{
				ConfigSection.Add(TEXT("bUseCultureDirectory"), TEXT("false"));
			}

			ConfigSection.Add(TEXT("ManifestName"), GetManifestFileName(Target));
			ConfigSection.Add(TEXT("ArchiveName"), GetArchiveFileName(Target));

			FString DialogueScriptFileName;
			// The import path for a specific culture is a file path.
			if (CultureName.IsSet() && ImportPathOverride.IsSet())
			{
				DialogueScriptFileName = FPaths::GetCleanFilename(ImportPathOverride.GetValue());
			}
			// Use the default PO file's name.
			else
			{
				DialogueScriptFileName = GetDefaultDialogueScriptFileName(Target);
			}
			ConfigSection.Add(TEXT("DialogueScriptName"), DialogueScriptFileName);
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - ImportDialogueScript
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add(TEXT("CommandletClass"), TEXT("ImportDialogueScript"));
			
			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetImportDialogueScriptConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ImportDialogueScript_%s.ini"), *Target->Settings.Name, *CultureName.GetValue());
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ImportDialogueScript.ini"), *Target->Settings.Name);
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateExportDialogueScriptConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName, const TOptional<FString> ExportPathOverride)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add(TEXT("SourcePath"), SourcePath);

			FString DestinationPath;
			// Overriding export path changes the destination directory for the dialogue script file.
			if (ExportPathOverride.IsSet())
			{
				// The output path for a specific culture is a file path.
				if (CultureName.IsSet())
				{
					DestinationPath = MakePathRelativeForCommandletProcess(FPaths::GetPath(ExportPathOverride.GetValue()), !bIsEngineTarget);
				}
				// Otherwise, it is a directory path.
				else
				{
					DestinationPath = MakePathRelativeForCommandletProcess(ExportPathOverride.GetValue(), !bIsEngineTarget);
				}
			}
			// Use the default dialogue script file's directory path.
			else
			{
				DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			}
			ConfigSection.Add(TEXT("DestinationPath"), DestinationPath);

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add(TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName);
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add(TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName);
			};

			// Export for a specific culture.
			if (CultureName.IsSet())
			{
				const int32 CultureIndex = Target->Settings.SupportedCulturesStatistics.IndexOfByPredicate([CultureName](const FCultureStatistics& Culture) { return Culture.CultureName == CultureName.GetValue(); });
				AddCultureToGenerate(CultureIndex);
			}
			// Export for all cultures.
			else
			{
				for (int32 CultureIndex = 0; CultureIndex < Target->Settings.SupportedCulturesStatistics.Num(); ++CultureIndex)
				{
					AddCultureToGenerate(CultureIndex);
				}
			}

			// Do not use culture subdirectories if exporting a single culture to a specific directory.
			if (CultureName.IsSet() && ExportPathOverride.IsSet())
			{
				ConfigSection.Add(TEXT("bUseCultureDirectory"), TEXT("false"));
			}

			ConfigSection.Add(TEXT("ManifestName"), GetManifestFileName(Target));
			ConfigSection.Add(TEXT("ArchiveName"), GetArchiveFileName(Target));

			FString DialogueScriptFileName;
			// The export path for a specific culture is a file path.
			if (CultureName.IsSet() && ExportPathOverride.IsSet())
			{
				DialogueScriptFileName = FPaths::GetCleanFilename(ExportPathOverride.GetValue());
			}
			// Use the default PO file's name.
			else
			{
				DialogueScriptFileName = GetDefaultDialogueScriptFileName(Target);
			}
			ConfigSection.Add(TEXT("DialogueScriptName"), DialogueScriptFileName);
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - ExportDialogueScript
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add(TEXT("CommandletClass"), TEXT("ExportDialogueScript"));
			
			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetExportDialogueScriptConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ExportDialogueScript_%s.ini"), *Target->Settings.Name, *CultureName.GetValue());
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ExportDialogueScript.ini"), *Target->Settings.Name);
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateImportDialogueConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add(TEXT("SourcePath"), SourcePath);

			ConfigSection.Add(TEXT("ManifestName"), GetManifestFileName(Target));
			ConfigSection.Add(TEXT("ArchiveName"), GetArchiveFileName(Target));

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add(TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName);
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add(TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName);
			};

			if (CultureName.IsSet())
			{
				const int32 CultureIndex = Target->Settings.SupportedCulturesStatistics.IndexOfByPredicate([CultureName](const FCultureStatistics& Culture) { return Culture.CultureName == CultureName.GetValue(); });
				AddCultureToGenerate(CultureIndex);
			}
			else
			{
				for (int32 CultureIndex = 0; CultureIndex < Target->Settings.SupportedCulturesStatistics.Num(); ++CultureIndex)
				{
					AddCultureToGenerate(CultureIndex);
				}
			}
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - ImportLocalizedDialogue
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add(TEXT("CommandletClass"), TEXT("ImportLocalizedDialogue"));

			ConfigSection.Add(TEXT("RawAudioPath"), Target->Settings.ImportDialogueSettings.RawAudioPath.Path);
			ConfigSection.Add(TEXT("ImportedDialogueFolder"), Target->Settings.ImportDialogueSettings.ImportedDialogueFolder);
			ConfigSection.Add(TEXT("bImportNativeAsSource"), Target->Settings.ImportDialogueSettings.bImportNativeAsSource ? TEXT("true") : TEXT("false"));

			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetImportDialogueConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ImportDialogue_%s.ini"), *Target->Settings.Name, *CultureName.GetValue());
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf(TEXT("%s_ImportDialogue.ini"), *Target->Settings.Name);
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateWordCountReportConfigFile(const ULocalizationTarget* const Target)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("SourcePath"), SourcePath );
			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("DestinationPath"), DestinationPath );

			ConfigSection.Add( TEXT("ManifestName"), GetManifestFileName(Target) );
			ConfigSection.Add( TEXT("ArchiveName"), GetArchiveFileName(Target) );

			for (const FCultureStatistics& CultureStatistics : Target->Settings.SupportedCulturesStatistics)
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), CultureStatistics.CultureName );
			}

			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - GenerateTextLocalizationReport
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GenerateTextLocalizationReport") );

			ConfigSection.Add( TEXT("bWordCountReport"), TEXT("true") );

			ConfigSection.Add( TEXT("WordCountReportName"), GetWordCountCSVFileName(Target) );

			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetWordCountReportConfigPath(const ULocalizationTarget* const Target)
	{
		return GetConfigDirectory(Target) / FString::Printf(TEXT("%s_GenerateReports.ini"), *Target->Settings.Name);
	}

	FLocalizationConfigurationScript GenerateCompileTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// CommonSettings
		{
			FConfigSection ConfigSection;

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("SourcePath"), SourcePath );
			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add( TEXT("DestinationPath"), DestinationPath );

			ConfigSection.Add( TEXT("ManifestName"), GetManifestFileName(Target) );
			ConfigSection.Add( TEXT("ArchiveName"), GetArchiveFileName(Target) );
			ConfigSection.Add( TEXT("ResourceName"), GetLocResFileName(Target) );

			ConfigSection.Add( TEXT("bSkipSourceCheck"), Target->Settings.CompileSettings.SkipSourceCheck ? TEXT("true") : TEXT("false") );
			ConfigSection.Add( TEXT("bValidateFormatPatterns"), Target->Settings.CompileSettings.ValidateFormatPatterns ? TEXT("true") : TEXT("false") );
			ConfigSection.Add( TEXT("bValidateSafeWhitespace"), Target->Settings.CompileSettings.ValidateSafeWhitespace ? TEXT("true") : TEXT("false") );
			ConfigSection.Add( TEXT("bValidateRichTextTags"), Target->Settings.CompileSettings.ValidateRichTextTags ? TEXT("true") : TEXT("false"));

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add( TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName );
			}

			const auto& AddCultureToGenerate = [&](const int32 Index)
			{
				ConfigSection.Add( TEXT("CulturesToGenerate"), Target->Settings.SupportedCulturesStatistics[Index].CultureName );
			};

			// Compile a specific culture.
			if (CultureName.IsSet())
			{
				const int32 CultureIndex = Target->Settings.SupportedCulturesStatistics.IndexOfByPredicate([CultureName](const FCultureStatistics& Culture) { return Culture.CultureName == CultureName.GetValue(); });
				AddCultureToGenerate(CultureIndex);
			}
			// Compile all cultures.
			else
			{
				for (int32 CultureIndex = 0; CultureIndex < Target->Settings.SupportedCulturesStatistics.Num(); ++CultureIndex)
			{
					AddCultureToGenerate(CultureIndex);
				}
			}
			
			Script.AddCommonSettings(MoveTemp(ConfigSection));
		}

		// GatherTextStep0 - GenerateTextLocalizationResource
		{
			FConfigSection ConfigSection;

			// CommandletClass
			ConfigSection.Add( TEXT("CommandletClass"), TEXT("GenerateTextLocalizationResource") );
			
			Script.AddGatherTextStep(0, MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetCompileTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName)
	{
		const FString ConfigFileDirectory = GetConfigDirectory(Target);
		FString ConfigFilePath;
		if (CultureName.IsSet())
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Compile_%s.ini"), *Target->Settings.Name, *CultureName.GetValue() );
		}
		else
		{
			ConfigFilePath = ConfigFileDirectory / FString::Printf( TEXT("%s_Compile.ini"), *Target->Settings.Name );
		}
		return ConfigFilePath;
	}

	FLocalizationConfigurationScript GenerateRegenerateResourcesConfigFile(const ULocalizationTarget* const Target)
	{
		FLocalizationConfigurationScript Script;

		const bool bIsEngineTarget = Target->IsMemberOfEngineTargetSet();
		const FString ContentDirRelativeToGameDir = MakePathRelativeForCommandletProcess(GetContentDir(Target), !bIsEngineTarget);

		// RegenerateResources
		{
			FConfigSection ConfigSection;

			if (Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex))
			{
				ConfigSection.Add(TEXT("NativeCulture"), Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName);
			}

			const FString SourcePath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add(TEXT("SourcePath"), SourcePath);
			const FString DestinationPath = ContentDirRelativeToGameDir / TEXT("Localization") / Target->Settings.Name;
			ConfigSection.Add(TEXT("DestinationPath"), DestinationPath);

			ConfigSection.Add(TEXT("ManifestName"), GetManifestFileName(Target));
			ConfigSection.Add(TEXT("ArchiveName"), GetArchiveFileName(Target));
			ConfigSection.Add(TEXT("ResourceName"), GetLocResFileName(Target));

			Script.Add(TEXT("RegenerateResources"), MoveTemp(ConfigSection));
		}

		Script.Dirty = true;

		return Script;
	}

	FString GetRegenerateResourcesConfigPath(const ULocalizationTarget* const Target)
	{
		return GetConfigDirectory(Target) / FString::Printf(TEXT("Regenerate%s.ini"), *(Target->Settings.Name));
	}
	
	bool WriteConfigFileWithSCC(const FString& InConfigFilename, FLocalizationConfigurationScript& InConfigScript)
	{
		FString NewConfigContents;
		{
			// We only want to write the file if it's changed, but our config system can only write to files, so we have to use a temporary file :(
			const FString TempConfigFilename = FPaths::CreateTempFilename(*(FPaths::ProjectIntermediateDir() / TEXT("Localization")), TEXT("Localization"));
			if (!InConfigScript.Write(TempConfigFilename))
			{
				return false;
			}

			// Read the contents of the temp file back and clean it up
			const bool bReadFile = FFileHelper::LoadFileToString(NewConfigContents, *TempConfigFilename);
			IFileManager::Get().Delete(*TempConfigFilename);
			if (!bReadFile)
			{
				return false;
			}
			else
			{
				NewConfigContents = TEXT("; THESE ARE GENERATED FILES, DO NOT EDIT DIRECTLY!\r\n; USE THE LOCALIZATION DASHBOARD IN THE UNREAL EDITOR TO EDIT THE CONFIGURATION\r\n") + NewConfigContents;
			}
		}

		bool bWriteFile = true;
		{
			FString OldConfigContents;
			if (FFileHelper::LoadFileToString(OldConfigContents, *InConfigFilename))
			{
				bWriteFile = !OldConfigContents.Equals(NewConfigContents, ESearchCase::CaseSensitive);
			}
		}

		if (bWriteFile)
		{
			LocalizationConfigSCC::PreWriteFile(InConfigFilename);
			const bool bWroteFile = FFileHelper::SaveStringToFile(NewConfigContents, *InConfigFilename);
			LocalizationConfigSCC::PostWriteFile(InConfigFilename);
			return bWroteFile;
		}

		return true;
	}
}
