// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationDelegates.h"

TMulticastDelegate<void(const FString& LocalizationTargetPath)> LocalizationDelegates::OnLocalizationTargetDataUpdated;
