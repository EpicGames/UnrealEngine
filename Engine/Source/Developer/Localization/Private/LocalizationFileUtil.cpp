// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationFileUtil.h"

#include "Hash/xxhash.h"
#include "HAL/FileManager.h"
#include "LocalizationSourceControlUtil.h"
#include "Memory/MemoryView.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"

namespace UE::Localization::FileUtil::Private
{
	
bool WriteFileIfModified(const FString& Filename, TFunctionRef<bool(const FString&)> WriteFunc, TFunctionRef<uint64(const FString&)> HashFunc, ILocFileNotifies* LocFileNotifies)
{
	const FString TempFilename = Filename + TEXTVIEW(".tmp");

	if (!WriteFunc(TempFilename))
	{
		return false;
	}

	IFileManager& FileManager = IFileManager::Get();

	ON_SCOPE_EXIT
	{
		FileManager.Delete(*TempFilename);
	};

	const uint64 CurrentHash = HashFunc(Filename);
	const uint64 NewHash = HashFunc(TempFilename);

	if (CurrentHash == NewHash && FileManager.FileExists(*Filename))
	{
		return true;
	}

	if (LocFileNotifies)
	{
		LocFileNotifies->PreFileWrite(Filename);
	}

	const bool bUpdatedFile = FileManager.Move(*Filename, *TempFilename);
	
	if (LocFileNotifies)
	{
		LocFileNotifies->PostFileWrite(Filename);
	}
	
	return bUpdatedFile;
}
	
}

uint64 UE::Localization::FileUtil::BinaryHash(const FString& Filename)
{
	FXxHash64Builder HashBuilder;
	if (FPaths::FileExists(Filename))
	{
		FFileHelper::LoadFileInBlocks(Filename, 
			[&HashBuilder](FMemoryView Block)
			{
				HashBuilder.Update(Block);
			});
	}
	return HashBuilder.Finalize().Hash;
}

uint64 UE::Localization::FileUtil::TextHash(const FString& Filename)
{
	FXxHash64Builder HashBuilder;
	if (FPaths::FileExists(Filename))
	{
		FFileHelper::LoadFileToStringWithLineVisitor(*Filename,
			[&HashBuilder](FStringView Line)
			{
				HashBuilder.Update(Line.GetData(), Line.Len() * sizeof(TCHAR));
			});
	}
	return HashBuilder.Finalize().Hash;
}

uint64 UE::Localization::FileUtil::PortableObjectHash(const FString& Filename)
{
	FXxHash64Builder HashBuilder;
	if (FPaths::FileExists(Filename))
	{
		// Don't include the PO file header in the hash as it contains transient information (like timestamps) that we don't care about
		bool bSkippingPOFileHeader = true;

		FFileHelper::LoadFileToStringWithLineVisitor(*Filename,
			[&bSkippingPOFileHeader, &HashBuilder](FStringView Line)
			{
				if (bSkippingPOFileHeader)
				{
					// PO file headers end on the first empty line in the file
					bSkippingPOFileHeader = !Line.IsEmpty();
					return;
				}
				HashBuilder.Update(Line.GetData(), Line.Len() * sizeof(TCHAR));
			});
	}
	return HashBuilder.Finalize().Hash;
}

bool UE::Localization::FileUtil::WriteFileIfModified(const FString& Filename, TFunctionRef<bool(const FString&)> WriteFunc, TFunctionRef<uint64(const FString&)> HashFunc)
{
	return Private::WriteFileIfModified(Filename, WriteFunc, HashFunc, nullptr);
}

bool UE::Localization::FileUtil::WriteFileIfModified(const FString& Filename, TFunctionRef<bool(const FString&)> WriteFunc, TFunctionRef<uint64(const FString&)> HashFunc, const TSharedPtr<FLocalizationSCC>& SourceControlInfo)
{
	FLocFileSCCNotifies LocFileNotifies{SourceControlInfo};
	return Private::WriteFileIfModified(Filename, WriteFunc, HashFunc, &LocFileNotifies);
}

bool UE::Localization::FileUtil::WriteFileIfModified(const FString& Filename, TFunctionRef<bool(const FString&)> WriteFunc, TFunctionRef<uint64(const FString&)> HashFunc, const TSharedPtr<ILocFileNotifies>& LocFileNotifies)
{
	return Private::WriteFileIfModified(Filename, WriteFunc, HashFunc, LocFileNotifies.Get());
}
