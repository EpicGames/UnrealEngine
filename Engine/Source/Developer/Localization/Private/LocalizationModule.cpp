// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationModule.h"
#include "LocalizationSettings.h"
#include "TextLocalizationResourceGenerator.h"

class FLocalizationModule : public ILocalizationModule
{
public:
	virtual bool HandleRegenLocCommand(const FString& InConfigFilePath) override
	{
		return FTextLocalizationResourceGenerator::GenerateLocResAndUpdateLiveEntriesFromConfig(InConfigFilePath, EGenerateLocResFlags::None);
	}

	virtual ULocalizationTarget* GetLocalizationTargetByName(FString TargetName, bool bIsEngineTarget) override
	{
		if (bIsEngineTarget)
		{
			ULocalizationTargetSet* EngineTargetSet = ULocalizationSettings::GetEngineTargetSet();
			for (ULocalizationTarget* Target : EngineTargetSet->TargetObjects)
			{
				if (Target->Settings.Name == TargetName)
				{
					return Target;
				}
			}
		}
		else
		{
			ULocalizationTargetSet* GameTargetSet = ULocalizationSettings::GetGameTargetSet();
			for (ULocalizationTarget* Target : GameTargetSet->TargetObjects)
			{
				if (Target->Settings.Name == TargetName)
				{
					return Target;
				}
			}
		}

		return nullptr;
	}
};

IMPLEMENT_MODULE(FLocalizationModule, Localization);
