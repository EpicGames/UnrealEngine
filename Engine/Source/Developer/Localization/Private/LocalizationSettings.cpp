// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizationSettings)

ULocalizationSettings::ULocalizationSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EngineTargetSet(ObjectInitializer.CreateDefaultSubobject<ULocalizationTargetSet>(this, TEXT("EngineLocalizationTargetSet")))
	, GameTargetSet(ObjectInitializer.CreateDefaultSubobject<ULocalizationTargetSet>(this, TEXT("ProjectLocalizationTargetSet")))
{	
}

#if WITH_EDITOR
void ULocalizationSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Create and initialize objects for details model from backing config properties.
	if (EngineTargetSet)
	{
		EngineTargetSet->TargetObjects.Empty(EngineTargetsSettings.Num());
		for (const auto& TargetSettings : EngineTargetsSettings)
		{
			ULocalizationTarget* const TargetObject = NewObject<ULocalizationTarget>(EngineTargetSet);
			TargetObject->Settings = TargetSettings;
			TargetObject->UpdateStatusFromConflictReport();
			TargetObject->UpdateWordCountsFromCSV();
			EngineTargetSet->TargetObjects.Add(TargetObject);
		}
	}

	// Create and initialize objects for details model from backing config properties.
	if (GameTargetSet)
	{
		GameTargetSet->TargetObjects.Empty(EngineTargetsSettings.Num());
		for (const auto& TargetSettings : GameTargetsSettings)
		{
			ULocalizationTarget* const TargetObject = NewObject<ULocalizationTarget>(GameTargetSet);
			TargetObject->Settings = TargetSettings;
			TargetObject->UpdateStatusFromConflictReport();
			TargetObject->UpdateWordCountsFromCSV();
			GameTargetSet->TargetObjects.Add(TargetObject);
		}
	}
}

void ULocalizationSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Copy settings back.
	if (EngineTargetSet)
	{
		EngineTargetsSettings.Empty(EngineTargetSet->TargetObjects.Num());
		for (const auto& TargetObject : EngineTargetSet->TargetObjects)
		{
			EngineTargetsSettings.Add(TargetObject ? TargetObject->Settings : FLocalizationTargetSettings());
		}
	}

	// Copy settings back.
	if (GameTargetSet)
	{
		GameTargetsSettings.Empty(GameTargetSet->TargetObjects.Num());
		for (const auto& TargetObject : GameTargetSet->TargetObjects)
		{
			GameTargetsSettings.Add(TargetObject ? TargetObject->Settings : FLocalizationTargetSettings());
		}
	}

	TryUpdateDefaultConfigFile();
}
#endif

ULocalizationTargetSet* ULocalizationSettings::GetEngineTargetSet()
{
	const ULocalizationSettings* LocalizationSettings = GetDefault<ULocalizationSettings>();
	check(LocalizationSettings);
	return LocalizationSettings->EngineTargetSet;
}

ULocalizationTargetSet* ULocalizationSettings::GetGameTargetSet()
{
	const ULocalizationSettings* LocalizationSettings = GetDefault<ULocalizationSettings>();
	check(LocalizationSettings);
	return LocalizationSettings->GameTargetSet;
}


const FString FLocalizationSourceControlSettings::LocalizationSourceControlSettingsCategoryName = TEXT("LocalizationSourceControlSettings");
const FString FLocalizationSourceControlSettings::SourceControlEnabledSettingName = TEXT("SourceControlEnabled");
const FString FLocalizationSourceControlSettings::SourceControlAutoSubmitEnabledSettingName = TEXT("SourceControlAutoSubmitEnabled");

bool FLocalizationSourceControlSettings::IsSourceControlAvailable()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	return SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable();
}

bool FLocalizationSourceControlSettings::IsSourceControlEnabled()
{
	if (!IsSourceControlAvailable())
	{
		return false;
	}

	bool bIsEnabled = true;
	if (!GConfig->GetBool(*LocalizationSourceControlSettingsCategoryName, *SourceControlEnabledSettingName, bIsEnabled, GEditorPerProjectIni))
	{
		bIsEnabled = true;
	}
	return bIsEnabled;
}

bool FLocalizationSourceControlSettings::IsSourceControlAutoSubmitEnabled()
{
	if (!IsSourceControlAvailable())
	{
		return false;
	}

	bool bIsEnabled = false;
	if (!GConfig->GetBool(*LocalizationSourceControlSettingsCategoryName, *SourceControlAutoSubmitEnabledSettingName, bIsEnabled, GEditorPerProjectIni))
	{
		bIsEnabled = false;
	}
	return bIsEnabled;
}

void FLocalizationSourceControlSettings::SetSourceControlEnabled(const bool bIsEnabled)
{
	GConfig->SetBool(*LocalizationSourceControlSettingsCategoryName, *SourceControlEnabledSettingName, bIsEnabled, GEditorPerProjectIni);
}

void FLocalizationSourceControlSettings::SetSourceControlAutoSubmitEnabled(const bool bIsEnabled)
{
	GConfig->SetBool(*LocalizationSourceControlSettingsCategoryName, *SourceControlAutoSubmitEnabledSettingName, bIsEnabled, GEditorPerProjectIni);
}

