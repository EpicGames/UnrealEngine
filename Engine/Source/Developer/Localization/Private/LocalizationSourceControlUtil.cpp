// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationSourceControlUtil.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Logging/StructuredLog.h"
#include "Misc/Paths.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"

DEFINE_LOG_CATEGORY_STATIC(LogLocalizationSourceControl, Log, All);
namespace LocalizationSourceControlUtil
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

#define LOCTEXT_NAMESPACE "LocalizationSourceControl"

FLocalizationSCC::FLocalizationSCC(const EInitSCC InitSCC)
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC must be created on the game-thread"));

	if (InitSCC == EInitSCC::Always || (InitSCC == EInitSCC::Commandlet && IsRunningCommandlet()))
	{
		ISourceControlModule::Get().GetProvider().Init();
		bDidInitSCC = true;
	}
}

FLocalizationSCC::~FLocalizationSCC()
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC must be destroyed on the game-thread"));

	if (CheckedOutFiles.Num() > 0)
	{
		UE_LOGF(LogLocalizationSourceControl, Log, "Revision Control wrapper shutting down with checked out files.");
	}

	if (bDidInitSCC)
	{
		ISourceControlModule::Get().GetProvider().Close();
	}
}

TSharedPtr<FLocalizationSCC> FLocalizationSCC::CreateIfSCCEnabled(const EInitSCC InitSCC)
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC::CreateIfSCCEnabled must be called on the game-thread"));

	if (ISourceControlModule::Get().IsEnabled())
	{
		return MakeShared<FLocalizationSCC>(InitSCC);
	}
	
	return nullptr;
}

void FLocalizationSCC::BeginParallelTasks()
{
	++ParallelTasksCount;
}

bool FLocalizationSCC::EndParallelTasks(FText& OutError)
{
	const uint16 PrevParallelTasksCount = ParallelTasksCount--;
	checkf(PrevParallelTasksCount > 0, TEXT("FLocalizationSCC::EndParallelTasks was called while no parallel tasks are running"));
	if (PrevParallelTasksCount > 1)
	{
		return true;
	}

	checkf(IsInGameThread(), TEXT("FLocalizationSCC::EndParallelTasks must be called on the game-thread when closing the final block"));

	if (DeferredCheckedOutFiles.IsEmpty())
	{
		return true;
	}

	TArray<FString> DeferredCheckedOutFilesArray = DeferredCheckedOutFiles.Array();
	DeferredCheckedOutFiles.Reset();

	if (!IsReady(OutError))
	{
		return false;
	}

	// Try and apply the deferred check-out requests
	if (!USourceControlHelpers::CheckOutOrAddFiles(DeferredCheckedOutFilesArray))
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// Figure out which files failed to check-out or add
		if (TArray<FSourceControlStateRef> SourceControlStates;
			SourceControlProvider.GetState(DeferredCheckedOutFilesArray, SourceControlStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
		{
			check(DeferredCheckedOutFilesArray.Num() == SourceControlStates.Num());

			TArray<FString> FilesToForceSync;
			for (int32 Index = 0; Index < DeferredCheckedOutFilesArray.Num(); ++Index)
			{
				FString& DeferredCheckedOutFile = DeferredCheckedOutFilesArray[Index];
				const FSourceControlStateRef& SourceControlState = SourceControlStates[Index];

				if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
				{
					CheckedOutFiles.Add(MoveTemp(DeferredCheckedOutFile));
				}
				else
				{
					FilesToForceSync.Add(MoveTemp(DeferredCheckedOutFile));
				}
			}

			if (FilesToForceSync.Num() > 0)
			{
				// Failed the deferred check-out, so try and restore the on-disk file to the source controlled state
				TSharedRef<FSync> ForceSyncOperation = ISourceControlOperation::Create<FSync>();
				ForceSyncOperation->SetForce(true);
				ForceSyncOperation->SetLastSyncedFlag(true);

				SourceControlProvider.Execute(ForceSyncOperation, FilesToForceSync);
			}

			OutError = FText::Format(LOCTEXT("EndParallelTasksFailed.WithFiles", "FLocalizationSCC::EndParallelTasks failed to check-out or add: {0}"), FText::FromString(FString::Join(FilesToForceSync, TEXT(", "))));
		}
		else
		{
			OutError = LOCTEXT("EndParallelTasksFailed.Generic", "FLocalizationSCC::EndParallelTasksFailed failed to check-out or add some files, but querying the file states also failed");
		}
		return false;
	}
	else
	{
		for (FString& DeferredCheckedOutFile : DeferredCheckedOutFilesArray)
		{
			CheckedOutFiles.Add(MoveTemp(DeferredCheckedOutFile));
		}
	}

	return true;
}

bool FLocalizationSCC::CheckOutFile(const FString& InFile, FText& OutError)
{
	if (InFile.IsEmpty())
	{
		OutError = LOCTEXT("InvalidFileSpecified", "Could not checkout file at invalid path.");
		return false;
	}

	if (InFile.StartsWith(TEXT("\\\\")))
	{
		// We can't check out a UNC path, but don't say we failed
		return true;
	}

	FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InFile);

	if (ParallelTasksCount > 0)
	{
		UE::TScopeLock _(FilesMutex);

		if (CheckedOutFiles.Contains(AbsoluteFilename) || DeferredCheckedOutFiles.Contains(AbsoluteFilename))
		{
			return true;
		}

		// Make the file writable on-disk and add it to the deferred set
		if (IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			PlatformFile.IsReadOnly(*AbsoluteFilename))
		{
			if (PlatformFile.SetReadOnly(*AbsoluteFilename, false))
			{
				DeferredCheckedOutFiles.Add(MoveTemp(AbsoluteFilename));
			}
			else
			{
				OutError = FText::Format(LOCTEXT("FailedToCheckOutFile", "Failed to make file writable '{0}'."), FText::FromString(AbsoluteFilename));
				return false;
			}
		}
		else
		{
			DeferredCheckedOutFiles.Add(MoveTemp(AbsoluteFilename));
		}

		return true;
	}

	if (!IsReady(OutError))
	{
		return false;
	}

	if (CheckedOutFiles.Contains(AbsoluteFilename))
	{
		return true;
	}

	if (!USourceControlHelpers::CheckOutOrAddFile(AbsoluteFilename))
	{
		OutError = USourceControlHelpers::LastErrorMsg();
		return false;
	}

	// Make sure the file is actually writable, as adding a read-only file to source control may leave it read-only
	if (IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.IsReadOnly(*AbsoluteFilename))
	{
		PlatformFile.SetReadOnly(*AbsoluteFilename, false);
	}

	CheckedOutFiles.Add(MoveTemp(AbsoluteFilename));

	return true;
}

bool FLocalizationSCC::CheckinFiles(const FText& InChangeDescription, FText& OutError)
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC::CheckinFiles must be called on the game-thread"));
	checkf(ParallelTasksCount == 0, TEXT("FLocalizationSCC::CheckinFiles was called while parallel tasks are running"));

	if (CheckedOutFiles.IsEmpty())
	{
		return true;
	}

	if (!IsReady(OutError))
	{
		return false;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Revert any unchanged files
	{
		TArray<FString> CheckedOutFilesArray = CheckedOutFiles.Array();
		USourceControlHelpers::RevertUnchangedFiles(SourceControlProvider, CheckedOutFilesArray);

		// Update CheckedOutFiles with the files that are still actually checked-out or added
		if (TArray<FSourceControlStateRef> SourceControlStates;
			SourceControlProvider.GetState(CheckedOutFilesArray, SourceControlStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
		{
			check(CheckedOutFilesArray.Num() == SourceControlStates.Num());
			
			CheckedOutFiles.Reset();
			for (int32 Index = 0; Index < CheckedOutFilesArray.Num(); ++Index)
			{
				FString& CheckedOutFile = CheckedOutFilesArray[Index];
				const FSourceControlStateRef& SourceControlState = SourceControlStates[Index];

				if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
				{
					CheckedOutFiles.Add(MoveTemp(CheckedOutFile));
				}
			}
		}
	}

	if (CheckedOutFiles.Num() > 0)
	{
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
		CheckInOperation->SetDescription(InChangeDescription);
		if (!SourceControlProvider.Execute(CheckInOperation, CheckedOutFiles.Array()))
		{
			OutError = LOCTEXT("FailedToCheckInFiles", "The checked out localization files could not be checked in.");
			return false;
		}
		CheckedOutFiles.Reset();
	}

	return true;
}

bool FLocalizationSCC::CleanUp(FText& OutError)
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC::CleanUp must be called on the game-thread"));
	checkf(ParallelTasksCount == 0, TEXT("FLocalizationSCC::CleanUp was called while parallel tasks are running"));

	if (CheckedOutFiles.IsEmpty())
	{
		return true;
	}

	if (!IsReady(OutError))
	{
		return false;
	}

	TArray<FString> CheckedOutFilesArray = CheckedOutFiles.Array();
	CheckedOutFiles.Reset();

	// Try and revert everything
	if (!USourceControlHelpers::RevertFiles(CheckedOutFilesArray))
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// Update CheckedOutFiles with the files that are still actually checked-out or added
		if (TArray<FSourceControlStateRef> SourceControlStates;
			SourceControlProvider.GetState(CheckedOutFilesArray, SourceControlStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
		{
			check(CheckedOutFilesArray.Num() == SourceControlStates.Num());

			for (int32 Index = 0; Index < CheckedOutFilesArray.Num(); ++Index)
			{
				FString& CheckedOutFile = CheckedOutFilesArray[Index];
				const FSourceControlStateRef& SourceControlState = SourceControlStates[Index];

				if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
				{
					CheckedOutFiles.Add(MoveTemp(CheckedOutFile));
				}
			}

			OutError = FText::Format(LOCTEXT("CleanUpFailed.WithFiles", "FLocalizationSCC::CleanUp failed to revert: {0}"), FText::FromString(FString::Join(CheckedOutFiles, TEXT(", "))));
		}
		else
		{
			OutError = LOCTEXT("CleanUpFailed.Generic", "FLocalizationSCC::CleanUp failed to revert some files, but querying the file states also failed");
		}
		return false;
	}

	return true;
}

bool FLocalizationSCC::IsReady(FText& OutError) const
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC::IsReady must be called on the game-thread"));
	checkf(ParallelTasksCount == 0, TEXT("FLocalizationSCC::IsReady was called while parallel tasks are running"));

	if (!ISourceControlModule::Get().IsEnabled())
	{
		OutError = LOCTEXT("SourceControlNotEnabled", "Revision control is not enabled.");
		return false;
	}

	if (!ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		OutError = LOCTEXT("SourceControlNotAvailable", "Revision control server is currently not available.");
		return false;
	}

	return true;
}

bool FLocalizationSCC::RevertFile(const FString& InFile, FText& OutError)
{
	checkf(IsInGameThread(), TEXT("FLocalizationSCC::RevertFile must be called on the game-thread"));
	checkf(ParallelTasksCount == 0, TEXT("FLocalizationSCC::RevertFile was called while parallel tasks are running"));

	if (InFile.IsEmpty() || InFile.StartsWith(TEXT("\\\\")))
	{
		OutError = LOCTEXT("CouldNotRevertFile", "Could not revert file.");
		return false;
	}

	if (!IsReady(OutError))
	{
		return false;
	}

	FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InFile);

	if (!USourceControlHelpers::RevertFile(AbsoluteFilename))
	{
		OutError = USourceControlHelpers::LastErrorMsg();
		return false;
	}

	CheckedOutFiles.Remove(AbsoluteFilename);

	return true;
}

void FLocFileSCCNotifies::BeginParallelTasks()
{
	if (SourceControlInfo)
	{
		SourceControlInfo->BeginParallelTasks();
	}
}

void FLocFileSCCNotifies::EndParallelTasks()
{
	if (SourceControlInfo)
	{
		FText ErrorMsg;
		if (!SourceControlInfo->EndParallelTasks(ErrorMsg))
		{
			UE_LOGFMT(LogLocalizationSourceControl, Error, "{error}",
				("error", ErrorMsg.ToString()),
				("id", LocalizationSourceControlUtil::LocalizationLogIdentifier)
			);
		}
	}
}

void FLocFileSCCNotifies::PreFileWrite(const FString& InFilename)
{
	if (SourceControlInfo && FPaths::FileExists(InFilename))
	{
		// File already exists, so check it out before writing to it
		FText ErrorMsg;
		if (!SourceControlInfo->CheckOutFile(InFilename, ErrorMsg))
		{
			UE_LOGFMT(LogLocalizationSourceControl, Error, "Failed to check out file '{file}'. {error}",
				("file", InFilename),
				("error", ErrorMsg.ToString()),
				("id", LocalizationSourceControlUtil::LocalizationLogIdentifier)
			);
		}
	}
}

void FLocFileSCCNotifies::PostFileWrite(const FString& InFilename)
{
	if (SourceControlInfo)
	{
		// If the file didn't exist before then this will add it, otherwise it will do nothing
		FText ErrorMsg;
		if (!SourceControlInfo->CheckOutFile(InFilename, ErrorMsg))
		{
			UE_LOGFMT(LogLocalizationSourceControl, Error, "Failed to check out file '{file}'. {error}",
				("file", InFilename),
				("error", ErrorMsg.ToString()),
				("id", LocalizationSourceControlUtil::LocalizationLogIdentifier)
			);
		}
	}
}

#undef LOCTEXT_NAMESPACE
