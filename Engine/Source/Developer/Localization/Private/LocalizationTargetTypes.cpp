// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationTargetTypes.h"
#include "Templates/Casts.h"
#include "HAL/FileManager.h"
#include "Misc/AsciiSet.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Serialization/Csv/CsvParser.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizationTargetTypes)

#define LOCTEXT_NAMESPACE "LocalizationTargetTypes"

FString FLocalizationGatherPathRootUtil::GetResolvedPathRootToken(const ELocalizationGatherPathRoot InPathRoot)
{
	switch (InPathRoot)
	{
	case ELocalizationGatherPathRoot::Engine:
		return TEXT("%LOCENGINEROOT%");
	case ELocalizationGatherPathRoot::Project:
		return TEXT("%LOCPROJECTROOT%");
	}

	return FString();
}

FString FLocalizationGatherPathRootUtil::GetResolvedPathRoot(const ELocalizationGatherPathRoot InPathRoot, const bool bIsEngineTarget)
{
	switch (InPathRoot)
	{
	case ELocalizationGatherPathRoot::Auto:
		return bIsEngineTarget ? FPaths::EngineDir() : FPaths::ProjectDir();
	case ELocalizationGatherPathRoot::Engine:
		return FPaths::EngineDir();
	case ELocalizationGatherPathRoot::Project:
		return FPaths::ProjectDir();
	}

	checkf(0, TEXT("Unknown ELocalizationGatherPathRoot!"));
	return FString();
}

FText FLocalizationGatherPathRootUtil::GetResolvedPathRootDisplayName(const ELocalizationGatherPathRoot InPathRoot, const bool bIsEngineTarget)
{
	auto CreateProjectRootDisplayName = []() -> FText
	{
		return FText::Format(LOCTEXT("ProjectRootDisplayNameFmt", "{0}/"), FText::AsCultureInvariant(FApp::GetProjectName()));
	};

	switch (InPathRoot)
	{
	case ELocalizationGatherPathRoot::Auto:
		return bIsEngineTarget ? LOCTEXT("EngineRootDisplayName", "Engine/") : CreateProjectRootDisplayName();
	case ELocalizationGatherPathRoot::Engine:
		return LOCTEXT("EngineRootDisplayName", "Engine/");
	case ELocalizationGatherPathRoot::Project:
		return CreateProjectRootDisplayName();
	}

	checkf(0, TEXT("Unknown ELocalizationGatherPathRoot!"));
	return FText();
}

bool FGatherTextSearchDirectory::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	if (bIsEngineTarget && PathRoot == ELocalizationGatherPathRoot::Project)
	{
		OutError = LOCTEXT("EngineCannotUseProjectRootError", "Engine targets cannot use 'Project' as their path root.");
		return false;
	}

	FString ResolvedPath = FPaths::IsRelative(Path) ? FPaths::Combine(*FLocalizationGatherPathRootUtil::GetResolvedPathRoot(PathRoot, bIsEngineTarget), *Path) : Path;
	if (ResolvedPath.IsEmpty())
	{
		OutError = LOCTEXT("SearchDirectoryEmptyError", "Search directory not specified. Use \".\" to specify the root directory.");
		return false;
	}

	{
		constexpr FAsciiSet Wildcards("*?");
		if (const TCHAR* FirstWildcard = FAsciiSet::FindFirstOrEnd(*ResolvedPath, Wildcards); *FirstWildcard != 0)
		{
			// Trim the wildcard from this search path, as it would fail the rest of the validation with it
			ResolvedPath = ResolvedPath.Left(UE_PTRDIFF_TO_INT32(FirstWildcard - *ResolvedPath));
			ResolvedPath = FPaths::GetPath(MoveTemp(ResolvedPath));
		}
	}

	FText InvalidPathReason;
	if (!FPaths::ValidatePath(ResolvedPath, &InvalidPathReason))
	{
		OutError = InvalidPathReason;
		return false;
	}

	if (!FPaths::DirectoryExists(ResolvedPath))
	{
		OutError = LOCTEXT("SearchDirectoryNonExistentError", "Search directory does not exist.");
		return false;
	}

	return true;
}

bool FGatherTextIncludePath::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	if (Pattern.IsEmpty())
	{
		OutError = LOCTEXT("IncludePathEmptyError", "Include path not specified. Use \".\" to specify the root directory.");
		return false;
	}

	if (!Pattern.Contains(TEXT("*")))
	{
		OutError = LOCTEXT("IncludePathNoWildcardError", "Include path does not specify a wild card (\"*\"). Append \"*\" or only the file at the exact specified directory will be gathered from.");
		return false;
	}

	return true;
}

bool FGatherTextExcludePath::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	if (Pattern.IsEmpty())
	{
		OutError = LOCTEXT("ExcludePathEmptyError", "Exclude path not specified. Use \".\" to specify the root directory.");
		return false;
	}

	if (!Pattern.Contains(TEXT("*")))
	{
		OutError = LOCTEXT("ExcludePathNoWildcardError", "Exclude path does not specify a wild card (\"*\"). Append \"*\" or only the file at the exact specified directory will be excluded.");
		return false;
	}

	return true;
}

bool FGatherTextFileExtension::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	if (Pattern.IsEmpty())
	{
		OutError = LOCTEXT("FileExtensionEmptyError", "File extension not specified.");
		return false;
	}

	return true;
}

const TArray<FGatherTextFileExtension>& FGatherTextFromTextFilesConfiguration::GetDefaultTextFileExtensions()
{
	static const TArray<FGatherTextFileExtension> DefaultTextFileExtensions = []()
		{
			TArray<FGatherTextFileExtension> Result;
			Result.Add(FGatherTextFileExtension{TEXT("h")});
			Result.Add(FGatherTextFileExtension{TEXT("cpp")});
			Result.Add(FGatherTextFileExtension{TEXT("ini")});
			return Result;
		}();
	return DefaultTextFileExtensions;
}

const TArray<FGatherTextFileExtension>& FGatherTextFromPackagesConfiguration::GetDefaultPackageFileExtensions()
{
	static const TArray<FGatherTextFileExtension> DefaultPackageFileExtensions = []()
		{
			TArray<FGatherTextFileExtension> Result;
			Result.Add(FGatherTextFileExtension{TEXT("umap")});
			Result.Add(FGatherTextFileExtension{TEXT("uasset")});
			return Result;
		}();
	return DefaultPackageFileExtensions;
}

bool FGatherTextFromTextFilesConfiguration::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	const auto& InvalidSearchDirectoryPredicate = [bIsEngineTarget](const FGatherTextSearchDirectory& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidExcludePathPredicate = [bIsEngineTarget](const FGatherTextExcludePath& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidFileExtensionPredicate = [bIsEngineTarget](const FGatherTextFileExtension& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};

	if (SearchDirectories.Num() > 0 && !SearchDirectories.ContainsByPredicate(InvalidSearchDirectoryPredicate) &&
		!ExcludePathWildcards.ContainsByPredicate(InvalidExcludePathPredicate) &&
		FileExtensions.Num() > 0 && !FileExtensions.ContainsByPredicate(InvalidFileExtensionPredicate))
	{
		return true;
	}

	OutError = LOCTEXT("InvalidGatherTextFromFilesConfigurationError", "Must have at least one search directory, one file extension, and no invalid settings.");
	return false;
}

bool FGatherTextFromPackagesConfiguration::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	const auto& InvalidIncludePathPredicate = [bIsEngineTarget](const FGatherTextIncludePath& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidExcludePathPredicate = [bIsEngineTarget](const FGatherTextExcludePath& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidFileExtensionPredicate = [bIsEngineTarget](const FGatherTextFileExtension& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidMetaDataKeySpecificationPredicate = [bIsEngineTarget](const FMetaDataKeyGatherSpecification& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};

	if (IncludePathWildcards.Num() > 0 && !IncludePathWildcards.ContainsByPredicate(InvalidIncludePathPredicate) &&
		!ExcludePathWildcards.ContainsByPredicate(InvalidExcludePathPredicate) &&
		FileExtensions.Num() > 0 && !FileExtensions.ContainsByPredicate(InvalidFileExtensionPredicate) &&
		!MetadataKeySpecifications.ContainsByPredicate(InvalidMetaDataKeySpecificationPredicate))
	{
		return true;
	}

	OutError = LOCTEXT("InvalidGatherTextFromPackagesConfigurationError", "Must have at least one include path, one file extension, and no invalid settings.");
	return false;
}

bool FMetaDataTextKeyPattern::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	for (const auto& PossiblePlaceHolder : GetPossiblePlaceHolders())
	{
		if (Pattern.Contains(PossiblePlaceHolder))
		{
			return true;
		}
	}

	OutError = LOCTEXT("NoMetaDataLocalizationKeyPlaceHolderError", "No place holders used. All generated keys will conflict!");
	return false;
}

const TArray<FString>& FMetaDataTextKeyPattern::GetPossiblePlaceHolders()
{
	static const TArray<FString> PossiblePlaceHolders = []() -> TArray<FString>
	{
		TArray<FString> Result;
		Result.Add(TEXT("{FieldPath}"));
		Result.Add(TEXT("{MetaDataValue}"));
		return Result;
	}();

	return PossiblePlaceHolders;
}

bool FMetaDataKeyName::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	if (Name.IsEmpty())
	{
		OutError = LOCTEXT("MetaDataKeyNameEmptyError", "Meta data key not specified.");
		return false;
	}

	return true;
}

bool FMetaDataKeyGatherSpecification::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	return MetaDataKey.Validate(bIsEngineTarget, OutError) && TextKeyPattern.Validate(bIsEngineTarget, OutError);
}

bool FGatherTextFromMetaDataConfiguration::Validate(const bool bIsEngineTarget, FText& OutError) const
{
	const auto& InvalidIncludePathPredicate = [bIsEngineTarget](const FGatherTextIncludePath& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidExcludePathPredicate = [bIsEngineTarget](const FGatherTextExcludePath& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};
	const auto& InvalidMetaDataKeySpecificationPredicate = [bIsEngineTarget](const FMetaDataKeyGatherSpecification& Element) -> bool
	{
		FText ElementError;
		return !(Element.Validate(bIsEngineTarget, ElementError));
	};

	if (IncludePathWildcards.Num() > 0 && !IncludePathWildcards.ContainsByPredicate(InvalidIncludePathPredicate) &&
		!ExcludePathWildcards.ContainsByPredicate(InvalidExcludePathPredicate) &&
		KeySpecifications.Num() > 0 && !KeySpecifications.ContainsByPredicate(InvalidMetaDataKeySpecificationPredicate))
	{
		return true;
	}

	OutError = LOCTEXT("InvalidGatherTextFromMetadataConfigurationError", "Must have at least one include path, one meta data key specification, and no invalid settings.");
	return false;
}

namespace
{
	class FWordCountCSVParser
	{
		typedef TArray<const TCHAR*> FRow;

	public:
		static bool Execute(FLocalizationTargetSettings& Target, const FString& CSVFilePath)
		{
			// Load CSV file to string.
			FString CSVString;
			if (!FFileHelper::LoadFileToString(CSVString, *CSVFilePath))
			{
				return false;
			}

			// Parse CSV file string.
			FCsvParser CSVParser(CSVString);
			const FCsvParser::FRows& CSVRows = CSVParser.GetRows();

			// Initialize.
			FWordCountCSVParser Instance(Target);

			// Parse header row.
			if ( !(CSVRows.Num() > 0) ||!Instance.ParseHeaderRow(CSVRows[0]) )
			{
				return false;
			}

			// Parse latest word count row.
			TMap<FString, uint32> WordCounts;
			if ( !Instance.ParseWordCountRow(CSVRows.Top(), WordCounts) )
			{
				return false;
			}

			for (FCultureStatistics& SupportedCultureStatistics : Target.SupportedCulturesStatistics)
			{
				uint32* SupportedWordCount = WordCounts.Find(SupportedCultureStatistics.CultureName);
				if (SupportedWordCount)
				{
					SupportedCultureStatistics.WordCount = *SupportedWordCount;
				}
			}

			return true;
		}

	private:
		FWordCountCSVParser(FLocalizationTargetSettings& InTarget)
		{
			MandatoryColumnNames.Add( TEXT("Date/Time") );
			MandatoryColumnNames.Add( TEXT("Word Count") );
		}

		bool ParseHeaderRow(const FRow& HeaderRow)
		{
			// Validate header row.
			const bool IsHeaderRowValid = [&]() -> bool
			{
				if (HeaderRow.Num() < MandatoryColumnNames.Num())
				{
					// Not enough columns present to represent the mandatory columns.
					return false;
				}
				for (int32 i = 0; i < MandatoryColumnNames.Num(); ++i)
				{
					if (!MandatoryColumnNames[i].Equals(HeaderRow[i]))
					{
						return false;
					}
				}

				return true;
			}();

			if (!IsHeaderRowValid)
			{
				return false;
			}

			// Parse culture names.
			for (int32 i = MandatoryColumnNames.Num(); i < HeaderRow.Num(); ++i)
			{
				CultureNames.Add(HeaderRow[i]);
			}

			return true;
		}

		bool ParseWordCountRow(const FRow& WordCountRow, TMap<FString, uint32>& OutCultureWordCounts)
		{
			// Validate row.
			bool IsRowValid = [&]() -> bool
			{
				if (WordCountRow.Num() != MandatoryColumnNames.Num() + CultureNames.Num())
				{
					return false;
				}

				return true;
			}();

			if (!IsRowValid)
			{
				return false;
			}

			// Parse time stamp.
			FDateTime TimeStamp;
			if (!FDateTime::Parse(WordCountRow[0], TimeStamp))
			{
				return false;
			}

			// Parse supported culture word counts.
			for (int32 j = MandatoryColumnNames.Num(); j < WordCountRow.Num(); ++j)
			{
				const uint32 WordCount = static_cast<uint32>( FCString::Atoi(WordCountRow[j]) );
				const int32 CultureIndex = j - MandatoryColumnNames.Num();
				OutCultureWordCounts.FindOrAdd(CultureNames[CultureIndex]) = WordCount;
			}

			return true;
		}

	private:
		TArray<FString> MandatoryColumnNames;
		TArray<FString> CultureNames;
	};
}

ULocalizationTarget::ULocalizationTarget(const FObjectInitializer& ObjectInitializer)
{
	if(IsTemplate())
	{
		Settings.Guid = FGuid();
	}
}

#if WITH_EDITOR
void ULocalizationTarget::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Notify parent of change.
	ULocalizationTargetSet* const LocalizationTargetSet = Cast<ULocalizationTargetSet>(GetOuter());
	if (LocalizationTargetSet)
	{
		LocalizationTargetSet->PostEditChange();
	}
}
#endif

bool ULocalizationTarget::IsMemberOfEngineTargetSet() const
{
	ULocalizationTargetSet* const TargetSet = CastChecked<ULocalizationTargetSet>(GetOuter());
	return TargetSet == ULocalizationSettings::GetEngineTargetSet();
}

bool ULocalizationTarget::UpdateWordCountsFromCSV()
{
	const FString CSVFilePath = LocalizationConfigurationScript::GetWordCountCSVPath(this);
	const bool Succeeded = FWordCountCSVParser::Execute(Settings, CSVFilePath);
	if (!Succeeded)
	{
		for (FCultureStatistics& CultureStatistics : Settings.SupportedCulturesStatistics)
		{
			CultureStatistics.WordCount = 0;
		}
	}
	return Succeeded;
}

void ULocalizationTarget::UpdateStatusFromConflictReport()
{
	const FString ConflictReportFilePath = LocalizationConfigurationScript::GetConflictReportPath(this);
	const int64 ConflictReportFileSize = IFileManager::Get().FileSize(*ConflictReportFilePath);

	switch (ConflictReportFileSize)
	{
	case INDEX_NONE:
		// Unknown
		Settings.ConflictStatus = ELocalizationTargetConflictStatus::Unknown;
		break;
	case 0:
		// Good
		Settings.ConflictStatus = ELocalizationTargetConflictStatus::Clear;
		break;
	default:
		// Bad
		Settings.ConflictStatus = ELocalizationTargetConflictStatus::ConflictsPresent;
		break;
	}
}

bool ULocalizationTarget::RenameTargetAndFiles(const FString& NewName)
{
	bool HasCompletelySucceeded = true;
	ISourceControlModule& SourceControl = ISourceControlModule::Get();
	ISourceControlProvider& SourceControlProvider = SourceControl.GetProvider();
	const bool CanUseSourceControl = SourceControl.IsEnabled() && SourceControlProvider.IsEnabled() && SourceControlProvider.IsAvailable();

	const auto& TryDelete = [&](const FString& Path) -> bool
	{
		const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Path, EStateCacheUsage::Use);

		// Should only use source control if the file is actually under source control.
		if (CanUseSourceControl && SourceControlState.IsValid() && !SourceControlState->CanAdd())
		{
			// File is already marked for deletion.
			if (SourceControlState->IsDeleted())
			{
				return true;
			}
			else 
			{
				// File is in some modified source control state, needs to be reverted.
				if (SourceControlState->IsAdded() || SourceControlState->IsCheckedOut())
				{
					if (SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), *Path) != ECommandResult::Succeeded)
					{
						return false;
					}
				}

				// File needs to be deleted.
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), *Path) == ECommandResult::Succeeded)
				{
					return true;
				}
			}
		}
		// Attempt local deletion.
		else if (IFileManager::Get().Delete(*Path, false, true))
		{
			return true;
		}

		return false;
	};

	// Delete configuration files.
	const TArray<FString>& ScriptPaths = LocalizationConfigurationScript::GetConfigPaths(this);
	for (const FString& ScriptPath : ScriptPaths)
	{
		if (!TryDelete(ScriptPath))
		{
			HasCompletelySucceeded = false;
		}
	}

	const TArray<FString> OldPaths = LocalizationConfigurationScript::GetOutputFilePaths(this);

	// Rename
	Settings.Name = NewName;

	const TArray<FString> NewPaths = LocalizationConfigurationScript::GetOutputFilePaths(this);

	// Rename data files.
	check(OldPaths.Num() == NewPaths.Num());
	for (int32 i = 0; i < OldPaths.Num(); ++i)
	{
		if (IFileManager::Get().Move(*NewPaths[i], *OldPaths[i], true, true, false, true))
		{
			if (CanUseSourceControl)
			{
				// Add new file.
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *NewPaths[i]) != ECommandResult::Succeeded)
				{
					HasCompletelySucceeded = false;
				}
			}

			bool ShouldUseSourceControl = false;

			if (CanUseSourceControl)
			{
				const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*OldPaths[i], EStateCacheUsage::Use);
				if (SourceControlState->IsSourceControlled())
				{
					ShouldUseSourceControl = true;
				}
			}

			// Use source control.
			if (ShouldUseSourceControl)
			{
				// Delete old directory/file.
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), *OldPaths[i]) != ECommandResult::Succeeded)
				{
					HasCompletelySucceeded = false;
				}
			}
			// Operate locally.
			else
			{
				// Delete old directory/file.
				if (	(FPaths::DirectoryExists(OldPaths[i]) && !IFileManager::Get().DeleteDirectory(*OldPaths[i], false, true)) ||
					(FPaths::FileExists(OldPaths[i]) && !IFileManager::Get().Delete(*OldPaths[i], false, true))	)
				{
					HasCompletelySucceeded = false;
				}
			}
		}
	}

	// Generate new configuration files.
	LocalizationConfigurationScript::GenerateAllConfigFiles(this);

	//if (CanUseSourceControl)
	//{
	//	HasCompletelySucceeded = HasCompletelySucceeded && SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *LocalizationConfigurationScript::GetGatherScriptPath(this)) == ECommandResult::Succeeded;
	//	HasCompletelySucceeded = HasCompletelySucceeded && SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *LocalizationConfigurationScript::GetImportScriptPath(this)) == ECommandResult::Succeeded;
	//	HasCompletelySucceeded = HasCompletelySucceeded && SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *LocalizationConfigurationScript::GetExportScriptPath(this)) == ECommandResult::Succeeded;
	//	HasCompletelySucceeded = HasCompletelySucceeded && SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *LocalizationConfigurationScript::GetReportScriptPath(this)) == ECommandResult::Succeeded;
	//}

	return HasCompletelySucceeded;
}

bool ULocalizationTarget::DeleteFiles(const FString* const Culture) const
{
	bool HasCompletelySucceeded = true;
	ISourceControlModule& SourceControl = ISourceControlModule::Get();
	ISourceControlProvider& SourceControlProvider = SourceControl.GetProvider();

	const FString DataDirectory = LocalizationConfigurationScript::GetDataDirectory(this) / (Culture ? *Culture : FString());

	// Remove data files from source control.
	{
		TArray<FString> FilesInDataDirectory;
		IFileManager::Get().FindFilesRecursive(FilesInDataDirectory, *DataDirectory, TEXT("*"), true, false);

		TArray<FSourceControlStateRef> SourceControlStates;
		ECommandResult::Type Result = SourceControlProvider.GetState( FilesInDataDirectory, SourceControlStates, EStateCacheUsage::ForceUpdate );
		if (Result == ECommandResult::Succeeded)
		{
			TArray<FString> FilesToRevert;
			TArray<FString> FilesToDelete;

			for (const FSourceControlStateRef& SourceControlState : SourceControlStates)
			{
				// Added/ files must be reverted.
				if (SourceControlState->IsAdded())
				{
					FilesToRevert.Add(SourceControlState->GetFilename());
				}
				// Edited files must be reverted then deleted.
				else if (SourceControlState->IsCheckedOut())
				{
					FilesToRevert.Add(SourceControlState->GetFilename());
					FilesToDelete.Add(SourceControlState->GetFilename());
				}
				// Other source controlled files must be deleted.
				else if (SourceControlState->IsSourceControlled() && !SourceControlState->IsDeleted())
				{
					FilesToDelete.Add(SourceControlState->GetFilename());
				}
			}

			const bool WasSuccessfullyReverted = FilesToRevert.Num() == 0 || SourceControlProvider.Execute( ISourceControlOperation::Create<FRevert>(), FilesToRevert ) == ECommandResult::Succeeded;
			const bool WasSuccessfullyDeleted = FilesToDelete.Num() == 0 || SourceControlProvider.Execute( ISourceControlOperation::Create<FDelete>(), FilesToDelete ) == ECommandResult::Succeeded;
			HasCompletelySucceeded = HasCompletelySucceeded && WasSuccessfullyReverted && WasSuccessfullyDeleted;
		}
	}

	// Delete data files.
	if (!IFileManager::Get().DeleteDirectory(*DataDirectory, false, true))
	{
		HasCompletelySucceeded = false;
	}

	// Delete configuration files if deleting the target.
	if (!Culture)
	{
		const TArray<FString>& ScriptPaths = LocalizationConfigurationScript::GetConfigPaths(this);

		// Remove script files from source control.
		{
			TArray<FSourceControlStateRef> SourceControlStates;
			ECommandResult::Type Result = SourceControlProvider.GetState( ScriptPaths, SourceControlStates, EStateCacheUsage::ForceUpdate );
			if (Result == ECommandResult::Succeeded)
			{
				TArray<FString> FilesToRevert;
				TArray<FString> FilesToDelete;

				for (const FSourceControlStateRef& SourceControlState : SourceControlStates)
				{
					// Added/ files must be reverted.
					if (SourceControlState->IsAdded())
					{
						FilesToRevert.Add(SourceControlState->GetFilename());
					}
					// Edited files must be reverted then deleted.
					else if (SourceControlState->IsCheckedOut())
					{
						FilesToRevert.Add(SourceControlState->GetFilename());
						FilesToDelete.Add(SourceControlState->GetFilename());
					}
					// Other source controlled files must be deleted.
					else if (SourceControlState->IsSourceControlled() && !SourceControlState->IsDeleted())
					{
						FilesToDelete.Add(SourceControlState->GetFilename());
					}
				}

				const bool WasSuccessfullyReverted = FilesToRevert.Num() == 0 || SourceControlProvider.Execute( ISourceControlOperation::Create<FRevert>(), FilesToRevert ) == ECommandResult::Succeeded;
				const bool WasSuccessfullyDeleted = FilesToDelete.Num() == 0 || SourceControlProvider.Execute( ISourceControlOperation::Create<FDelete>(), FilesToDelete ) == ECommandResult::Succeeded;
				HasCompletelySucceeded = HasCompletelySucceeded && WasSuccessfullyReverted && WasSuccessfullyDeleted;
			}
		}

		// Delete script files.
		for (const FString& ScriptPath : ScriptPaths)
		{
			if (!IFileManager::Get().Delete(*ScriptPath, false, true))
			{
				HasCompletelySucceeded = false;
			}
		}
	}

	return HasCompletelySucceeded;
}

#if WITH_EDITOR
void ULocalizationTargetSet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Notify parent of change.
	ULocalizationSettings* const LocalizationSettings = Cast<ULocalizationSettings>(GetOuter());
	if (LocalizationSettings)
	{
		LocalizationSettings->PostEditChange();
	}
}
#endif

#undef LOCTEXT_NAMESPACE

