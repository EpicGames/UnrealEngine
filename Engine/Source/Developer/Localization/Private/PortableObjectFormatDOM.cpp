// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortableObjectFormatDOM.h"
#include "Internationalization/Culture.h"
#include "Containers/MapBuilder.h"

#define LOCTEXT_NAMESPACE "PortableObjectFormatDOM"

static const TCHAR* NewLineDelimiter = TEXT("\n");

/**
 * Default culture plural rules.  Culture names are in the following format: Language_Country@Variant
 * See references:	http://www.unicode.org/cldr/charts/latest/supplemental/language_plural_rules.html  
					http://docs.translatehouse.org/projects/localization-guide/en/latest/l10n/pluralforms.html
*/
const TCHAR* GetPluralForm(const TCHAR* InCulture)
{
	struct FPOCulturePluralForm
	{
		const TCHAR* CultureName;
		const TCHAR* PluralForm;
	};

	static const FPOCulturePluralForm POCulturePluralForms[] = {
		{ TEXT("ach"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("af"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ak"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("aln"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("am"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("am_ET"),	TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("an"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ar"),		TEXT("nplurals=6; plural=(n==0 ? 0 : n==1 ? 1 : n==2 ? 2 : n%100>=3 && n%100<=10 ? 3 : n%100>=11 && n%100<=99 ? 4 : 5);") },
		{ TEXT("ar_SA"),	TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("arn"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("as"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ast"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ay"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("az"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("bal"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("be"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("bg"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("bn"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("bo"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("br"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("brx"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("bs"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("ca"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("cgg"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("crh"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("cs"),		TEXT("nplurals=3; plural=(n==1) ? 0 : (n>=2 && n<=4) ? 1 : 2;") },
		{ TEXT("csb"),		TEXT("nplurals=3; plural=(n==1 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2;") },
		{ TEXT("cy"),		TEXT("nplurals=4; plural=(n==1) ? 0 : (n==2) ? 1 : (n != 8 && n != 11) ? 2 : 3;") },
		{ TEXT("da"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("de"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("doi"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("dz"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("el"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("en"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("eo"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("es"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("es_ar"),	TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("et"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("eu"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("fa"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("fi"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("fil"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("fo"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("fr"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("frp"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("fur"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("fy"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ga"),		TEXT("nplurals=5; plural=(n==1 ? 0 : n==2 ? 1 : n<7 ? 2 : n<11 ? 3 : 4);") },
		{ TEXT("gd"),		TEXT("nplurals=3; plural=(n < 2 ? 0 : n == 2 ? 1 : 2);") },
		{ TEXT("gl"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("gu"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("gun"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("ha"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("he"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("hi"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("hne"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("hr"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("hsb"),		TEXT("nplurals=4; plural=(n%100==1 ? 0 : n%100==2 ? 1 : n%100==3 || n%100==4 ? 2 : 3);") },
		{ TEXT("ht"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("hu"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("hy"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ia"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("id"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ig"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ilo"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("is"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("it"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ja"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("jv"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ka"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("kk"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("km"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("kn"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ko"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ks"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ku"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("kw"),		TEXT("nplurals=4; plural=(n==1) ? 0 : (n==2) ? 1 : (n == 3) ? 2 : 3;") },
		{ TEXT("ky"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("la"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("lb"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("li"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ln"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("lo"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("lt"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("lv"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n != 0 ? 1 : 2);") },
		{ TEXT("mai"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("mg"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("mi"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("mk"),		TEXT("nplurals=2; plural=(n % 10 == 1 && n % 100 != 11) ? 0 : 1;") },
		{ TEXT("ml"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("mn"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("mr"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ms"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("mt"),		TEXT("nplurals=4; plural=(n==1 ? 0 : n==0 || ( n%100>1 && n%100<11) ? 1 : (n%100>10 && n%100<20 ) ? 2 : 3);") },
		{ TEXT("my"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("nah"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nap"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nb"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nds"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ne"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nl"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nn"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("no"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("nr"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("nso"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("oc"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("or"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("pa"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("pap"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("pl"),		TEXT("nplurals=3; plural=(n==1 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("pms"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ps"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("pt"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("pt_BR"),	TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("rm"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ro"),		TEXT("nplurals=3; plural=(n==1?0:(((n%100>19)||((n%100==0)&&(n!=0)))?2:1));") },
		{ TEXT("ru"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("rw"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("sc"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("sco"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("se"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("si"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("sk"),		TEXT("nplurals=3; plural=(n==1) ? 0 : (n>=2 && n<=4) ? 1 : 2;") },
		{ TEXT("sl"),		TEXT("nplurals=4; plural=(n%100==1 ? 0 : n%100==2 ? 1 : n%100==3 || n%100==4 ? 2 : 3);") },
		{ TEXT("sm"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("sn"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("so"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("son"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("sq"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("sr"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("st"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("su"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("sv"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("sw"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("ta"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("te"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("tg"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("th"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ti"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("tk"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("tl"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("tlh"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("to"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("tr"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("tt"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("udm"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ug"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("uk"),		TEXT("nplurals=3; plural=(n%10==1 && n%100!=11 ? 0 : n%10>=2 && n%10<=4 && (n%100<10 || n%100>=20) ? 1 : 2);") },
		{ TEXT("ur"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("uz"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("ve"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("vi"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("vls"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("wa"),		TEXT("nplurals=2; plural=(n > 1);") },
		{ TEXT("wo"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("xh"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("yi"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("yo"),		TEXT("nplurals=2; plural=(n != 1);") },
		{ TEXT("zh"),		TEXT("nplurals=1; plural=0;") },
		{ TEXT("zu"),		TEXT("nplurals=2; plural=(n != 1);") },
	};

	for (const FPOCulturePluralForm& POCulturePluralForm : POCulturePluralForms)
	{
		if (FCString::Stricmp(POCulturePluralForm.CultureName, InCulture) == 0)
		{
			return POCulturePluralForm.PluralForm;
		}
	}

	return nullptr;
}

bool FindDelimitedString(const FString& InStr, const FString& LeftDelim, const FString& RightDelim, FString& Result)
{
	int32 Start = InStr.Find(LeftDelim, ESearchCase::CaseSensitive);
	int32 End = InStr.Find(RightDelim, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
	if (Start == INDEX_NONE || !(End > Start))
	{
		return false;
	}
	Start += LeftDelim.Len();
	if (Start >= End)
	{
		Result = TEXT("");
	}
	else
	{
		Result = InStr.Mid(Start, End - Start);
	}

	return true;
}

FPortableObjectCulture::FPortableObjectCulture( const FString& LangCode, const FString& PluralForms )
	: LanguageCode( LangCode )
	, LanguagePluralForms( PluralForms )
	, Culture( FInternationalization::Get().GetCulture( LangCode ) )
{
	
}

void FPortableObjectCulture::SetLanguageCode( const FString& LangCode )
{
	LanguageCode = LangCode;
	Culture = FInternationalization::Get().GetCulture( LangCode );
}

FString FPortableObjectCulture::Language() const
{
	FString Result;
	if( Culture.IsValid() )
	{
		// NOTE: The below function name may be a bit misleading because it will return three letter language codes if needed.
		Result = Culture->GetTwoLetterISOLanguageName();
	}
	return Result;

}

FString FPortableObjectCulture::Country() const
{
	FString Result;
	if( Culture.IsValid() )
	{
		Result = Culture->GetRegion();
	}
	return Result;
}

FString FPortableObjectCulture::Variant() const
{
	FString Result;
	if( Culture.IsValid() )
	{
		Result = Culture->GetVariant();
	}
	return Result;
}

FString FPortableObjectCulture::DisplayName() const
{
	FString Result;
	if( Culture.IsValid() )
	{
		Result = Culture->GetDisplayName();
	}
	return Result;
}

FString FPortableObjectCulture::EnglishName() const
{
	FString Result;
	if( Culture.IsValid() )
	{
		Result = Culture->GetEnglishName();
	}
	return Result;
}

FString FPortableObjectCulture::GetPluralForms() const
{
	if( !LanguagePluralForms.IsEmpty() )
	{
		return LanguagePluralForms;
	}
	return GetDefaultPluralForms();
}

FString FPortableObjectCulture::GetDefaultPluralForms() const
{
	FString Result;
	if( LanguageCode.IsEmpty() )
	{
		return Result;
	}

	if( const TCHAR* LanguageCodePair = GetPluralForm( *GetLanguageCode() ) )
	{
		Result = LanguageCodePair;
	}
	else if( const TCHAR* LangCountryVariantPair = GetPluralForm( *( Language() + TEXT("_") + Country() + TEXT("@") + Variant() ) ) )
	{
		Result = LangCountryVariantPair;
	}
	else if( const TCHAR* LangCountryPair = GetPluralForm( *( Language() + TEXT("_") + Country() ) ) )
	{
		Result = LangCountryPair;
	} 
	else if( const TCHAR* LangPair = GetPluralForm( *Language() ) )
	{
		Result = LangPair;
	}
	else
	{
		const TCHAR* Fallback = GetPluralForm( TEXT("en") );
		if( Fallback )
		{
			Result = Fallback;
		}
		else
		{
			Result = TEXT("nplurals=2; plural=(n != 1);");
		}
	}

	return Result;
}


FString FPortableObjectHeader::ToString() const
{
	FString Result;
	for( const FString& Comment : Comments )
	{
		Result += FString::Printf(TEXT("# %s%s"), *Comment, NewLineDelimiter);
	}

	Result += FString::Printf(TEXT("msgid \"\"%s"), NewLineDelimiter);
	Result += FString::Printf(TEXT("msgstr \"\"%s"), NewLineDelimiter);

	for( auto Entry : HeaderEntries )
	{
		const FString& Key = Entry.Key;
		const FString& Value = Entry.Value;
		Result += FString::Printf( TEXT("\"%s: %s\\n\"%s"), *Key, *Value, NewLineDelimiter );
	}

	return Result;
}

bool FPortableObjectHeader::FromLocPOEntry( const TSharedRef<const FPortableObjectEntry> LocEntry )
{
	if( !LocEntry->MsgId.IsEmpty() || LocEntry->MsgStr.Num() != 1 )
	{
		return false;
	}
	Clear();

	Comments = LocEntry->TranslatorComments;

	// The POEntry would store header info inside the MsgStr[0]
	TArray<FString> HeaderLinesToProcess;
	LocEntry->MsgStr[0].ReplaceEscapedCharWithChar().ParseIntoArray( HeaderLinesToProcess, NewLineDelimiter, true );

	for( const FString& PotentialHeaderEntry : HeaderLinesToProcess )
	{
		int32 SplitIndex;
		if( PotentialHeaderEntry.FindChar( TCHAR(':'), SplitIndex ) )
		{
			// Looks like a header entry so we add it
			const FString& Key = PotentialHeaderEntry.LeftChop( PotentialHeaderEntry.Len() - SplitIndex ).TrimStartAndEnd();
			FString Value = PotentialHeaderEntry.RightChop( SplitIndex+1 ).TrimStartAndEnd();

			HeaderEntries.Emplace( Key, MoveTemp(Value) );
		}
	}
	return true;
}

FPortableObjectHeader::FPOHeaderEntry* FPortableObjectHeader::FindEntry( const FString& EntryKey )
{
	for( auto& Entry : HeaderEntries )
	{
		if( Entry.Key == EntryKey )
		{
			return &Entry;
		}
	}
	return NULL;
}

const FPortableObjectHeader::FPOHeaderEntry* FPortableObjectHeader::FindEntry( const FString& EntryKey ) const
{
	for( auto& Entry : HeaderEntries )
	{
		if( Entry.Key == EntryKey )
		{
			return &Entry;
		}
	}
	return NULL;
}

FString FPortableObjectHeader::GetEntryValue( const FString& EntryKey ) const
{
	FString Result;
	const FPOHeaderEntry* Entry = FindEntry( EntryKey );
	if( Entry )
	{
		Result = Entry->Value;
	}
	return Result;
}

bool FPortableObjectHeader::HasEntry( const FString& EntryKey ) const
{
	return ( FindEntry( EntryKey ) != NULL);
}

void FPortableObjectHeader::SetEntryValue( const FString& EntryKey, const FString& EntryValue )
{
	FPOHeaderEntry* Entry = FindEntry( EntryKey );
	if( Entry )
	{
		Entry->Value = EntryValue;
	}
	else
	{
		HeaderEntries.Emplace( EntryKey, EntryValue );
	}
}

void FPortableObjectHeader::UpdateTimeStamp()
{
	// @TODO: Time format not exactly correct.  We have something like this: 2014-02-07 20:06  This is what it should be: 2014-02-07 14:12-0600
	FString Time = *FDateTime::UtcNow().ToString(TEXT("%Y-%m-%d %H:%M"));

	SetEntryValue( TEXT("POT-Creation-Date"), Time );
	SetEntryValue( TEXT("PO-Revision-Date"), Time );
}

FString FPortableObjectFormatDOM::ToString()
{
	FString Result;
	ToString(Result);
	return Result;
}

void FPortableObjectFormatDOM::ToString(FString& OutString)
{
	OutString.Reset();

	Header.UpdateTimeStamp();

	OutString += Header.ToString();
	OutString += NewLineDelimiter;

	for( const auto& EntryPair : Entries )
	{
		OutString += EntryPair.Value->ToString();
		OutString += NewLineDelimiter;
	}
}

bool FPortableObjectFormatDOM::FromString( const FString& InStr, FText* OutErrorMsg )
{
	auto SetErrorMsg = [OutErrorMsg](const FText& InErrorMsg)
	{
		if (OutErrorMsg)
		{
			*OutErrorMsg = InErrorMsg;
		}
	};

	if( InStr.IsEmpty() )
	{
		SetErrorMsg(LOCTEXT("Error_EmptyPOFile", "The PO/POT source was empty."));
		return false;
	}

	FString ParseString = InStr;
	ParseString.ReplaceInline(TEXT("\r\n"), NewLineDelimiter, ESearchCase::CaseSensitive);

	TArray<FString> LinesToProcess;
	ParseString.ParseIntoArray( LinesToProcess, NewLineDelimiter, false );

	TSharedRef<FPortableObjectEntry> ProcessedEntry = MakeShareable( new FPortableObjectEntry );
	bool bHasMsgId = false;

	uint32 NumFileLines = LinesToProcess.Num();
	for( uint32 LineIdx = 0; LineIdx < NumFileLines; ++LineIdx )
	{
		const FString& Line = LinesToProcess[LineIdx];

		auto SetFailedToParseQuoteDelimitedStringErrorMsg = [&SetErrorMsg, LineIdx, &Line]()
		{
			SetErrorMsg(FText::Format(LOCTEXT("Error_FailedToParseQuoteDelimitedString", "Failed to parse a quote-delimited string from line {0}: {1}"), LineIdx + 1, FText::AsCultureInvariant(Line)));
		};

		if( Line.IsEmpty() )
		{
			// When we encounter a blank line, we will either ignore it, or consider it the boundary of 
			// a LocPOEntry or FPortableObjectHeader if we processed any valid data before encountering the blank.

			// If this entry is valid we'll check it and process it further.
			if( bHasMsgId && ProcessedEntry->MsgStr.Num() > 0 )
			{
				// Check if we are dealing with a header entry
				if( ProcessedEntry->MsgId.IsEmpty() && ProcessedEntry->MsgStr.Num() == 1 )
				{
					// This is a header
					bool bAddedHeader = Header.FromLocPOEntry( ProcessedEntry );
					if( !bAddedHeader )
					{
						SetErrorMsg(LOCTEXT("Error_FailedToAddHeader", "Failed to add PO/POT header information to the DOM."));
						return false;
					}
					ProjectName = Header.GetEntryValue(TEXT("Project-Id-Version"));
				}
				else
				{
					bool bAddEntry = AddEntry( ProcessedEntry );
					if( !bAddEntry )
					{
						SetErrorMsg(LOCTEXT("Error_FailedToAddEntry", "Failed to add PO/POT entry information to the DOM."));
						return false;
					}
				}
			}

			// Now starting a new entry and reset any flags we have been tracking
			ProcessedEntry = MakeShareable( new FPortableObjectEntry );
			bHasMsgId = false;
		}
		else if( Line.StartsWith( TEXT("#,") ) )
		{
			// Flags
			FString Flag;
			if( FindDelimitedString(Line, TEXT("#,"), NewLineDelimiter, Flag ) )
			{
				if( !Flag.IsEmpty() )
				{
					ProcessedEntry->Flags.Add( Flag );
				}
			}
		}
		else if( Line.StartsWith( TEXT("#.") ) )
		{
			// Extracted comments
			FString Comment;
			ProcessedEntry->ExtractedComments.Add( Line.RightChop( FString(TEXT("#. ")).Len() ) );
		}
		else if( Line.StartsWith( TEXT("#:") ) )
		{
			// Reference
			FString Reference;
			ProcessedEntry->AddReference( Line.RightChop( FString(TEXT("#: ")).Len() ) );
		}
		else if( Line.StartsWith( TEXT(":|") ) )
		{
			// This represents previous messages. We just drop this in unknown since we don't handle it
			ProcessedEntry->UnknownElements.Add( Line );
		}
		else if( Line.StartsWith( TEXT("# ") ) || Line.StartsWith( TEXT("#\t") ) )
		{
			FString Comment;
			ProcessedEntry->TranslatorComments.Add( Line.RightChop( FString(TEXT("# ")).Len() ) );
		}
		else if( Line == TEXT("#") )
		{
			ProcessedEntry->TranslatorComments.Add( TEXT("") );		
		}
		else if( Line.StartsWith( TEXT("msgctxt") ) )
		{
			FString RawMsgCtxt;
			if( !FindDelimitedString(Line, TEXT("\""), TEXT("\""), RawMsgCtxt ) )
			{
				SetFailedToParseQuoteDelimitedStringErrorMsg();
				return false;
			}
			for( uint32 NextLineIdx = LineIdx + 1; NextLineIdx < NumFileLines && LinesToProcess[NextLineIdx].TrimStartAndEnd().StartsWith(TEXT("\"")); ++NextLineIdx)
			{
				FString Tmp;
				if (FindDelimitedString(Line, TEXT("\""), TEXT("\""), Tmp))
				{
					RawMsgCtxt += Tmp;
				}
			}

			// If the following line contains more info for this element we continue to parse it out
			uint32 NextLineIdx = LineIdx + 1;
			while( NextLineIdx < NumFileLines )
			{
				LinesToProcess[NextLineIdx].TrimStartAndEndInline();
				const FString& NextLine = LinesToProcess[NextLineIdx];
				if( NextLine.StartsWith("\"") && NextLine.EndsWith("\"") )
				{
					RawMsgCtxt += NextLine.Mid( 1, NextLine.Len()-2 );
				}
				else
				{
					break;
				}
				LineIdx = NextLineIdx;	
				NextLineIdx++;
			}

			ProcessedEntry->MsgCtxt = RawMsgCtxt;
		}
		else if( Line.StartsWith( TEXT("msgid") ) )
		{
			FString RawMsgId;
			if( !FindDelimitedString(Line, TEXT("\""), TEXT("\""), RawMsgId ) )
			{
				SetFailedToParseQuoteDelimitedStringErrorMsg();
				return false;
			}
			// If the following line contains more info for this element we continue to parse it out
			uint32 NextLineIdx = LineIdx + 1;
			while( NextLineIdx < NumFileLines )
			{
				LinesToProcess[NextLineIdx].TrimStartAndEndInline();
				const FString& NextLine = LinesToProcess[NextLineIdx];
				if( NextLine.StartsWith("\"") && NextLine.EndsWith("\"") )
				{
					RawMsgId += NextLine.Mid( 1, NextLine.Len()-2 );
				}
				else
				{
					break;
				}
				LineIdx = NextLineIdx;	
				NextLineIdx++;
			}
			ProcessedEntry->MsgId = RawMsgId;
			bHasMsgId = true;
		}
		else if( Line.StartsWith( TEXT("msgid_plural") ) )
		{
			FString RawMsgIdPlural;
			if( !FindDelimitedString(Line, TEXT("\""), TEXT("\""), RawMsgIdPlural ) )
			{
				SetFailedToParseQuoteDelimitedStringErrorMsg();
				return false;
			}
			// If the following line contains more info for this element we continue to parse it out
			uint32 NextLineIdx = LineIdx + 1;
			while( NextLineIdx < NumFileLines )
			{
				LinesToProcess[NextLineIdx].TrimStartAndEndInline();
				const FString& NextLine = LinesToProcess[NextLineIdx];
				if( NextLine.StartsWith("\"") && NextLine.EndsWith("\"") )
				{
					RawMsgIdPlural += NextLine.Mid( 1, NextLine.Len()-2 );
				}
				else
				{
					break;
				}
				LineIdx = NextLineIdx;	
				NextLineIdx++;
			}
			ProcessedEntry->MsgIdPlural = RawMsgIdPlural;
		}
		else if( Line.StartsWith( TEXT("msgstr[") ) )
		{
			FString IndexStr;
			if( !FindDelimitedString(Line, TEXT("["), TEXT("]"), IndexStr ) )
			{
				SetErrorMsg(FText::Format(LOCTEXT("Error_FailedToParseSquareBracketDelimitedString", "Failed to parse a square bracket-delimited string from line {0}: {1}"), LineIdx + 1, FText::AsCultureInvariant(Line)));
				return false;
			}
			int32 Index = -1;
			TTypeFromString<int32>::FromString(Index, *IndexStr );

			check(Index >= 0);

			FString RawMsgStr;
			if( !FindDelimitedString(Line, TEXT("\""), TEXT("\""), RawMsgStr ) )
			{
				SetFailedToParseQuoteDelimitedStringErrorMsg();
				return false;
			}
			// If the following line contains more info for this element we continue to parse it out
			uint32 NextLineIdx = LineIdx + 1;
			while( NextLineIdx < NumFileLines )
			{
				LinesToProcess[NextLineIdx].TrimStartAndEndInline();
				const FString& NextLine = LinesToProcess[NextLineIdx];
				if( NextLine.StartsWith("\"") && NextLine.EndsWith("\"") )
				{
					RawMsgStr += NextLine.Mid( 1, NextLine.Len()-2 );
				}
				else
				{
					break;
				}
				LineIdx = NextLineIdx;	
				NextLineIdx++;
			}

			if( ProcessedEntry->MsgStr.Num() > Index )
			{
				ProcessedEntry->MsgStr[Index] = RawMsgStr;
			}
			else
			{
				ProcessedEntry->MsgStr.Insert( RawMsgStr, Index );
			}
		}
		else if( Line.StartsWith( TEXT("msgstr") ) )
		{
			FString RawMsgStr;
			if( !FindDelimitedString(Line, TEXT("\""), TEXT("\""), RawMsgStr ) )
			{
				SetFailedToParseQuoteDelimitedStringErrorMsg();
				return false;
			}
			// If the following line contains more info for this element we continue to parse it out
			uint32 NextLineIdx = LineIdx + 1;
			while( NextLineIdx < NumFileLines )
			{
				LinesToProcess[NextLineIdx].TrimStartAndEndInline();
				const FString& NextLine = LinesToProcess[NextLineIdx];
				if( NextLine.StartsWith("\"") && NextLine.EndsWith("\"") )
				{
					RawMsgStr += NextLine.Mid( 1, NextLine.Len()-2 );
				}
				else
				{
					break;
				}
				LineIdx = NextLineIdx;	
				NextLineIdx++;
			}

			FString MsgStr = RawMsgStr;
			if( ProcessedEntry->MsgStr.Num() > 0 )
			{
				ProcessedEntry->MsgStr[0] = MsgStr;
			}
			else
			{
				ProcessedEntry->MsgStr.Add( MsgStr );
			}
		}
		else
		{
			ProcessedEntry->UnknownElements.Add( Line );
		}
	}

	return true;
}

void FPortableObjectFormatDOM::CreateNewHeader(  const FString& InCopyrightNotice )
{
	// Reference: http://www.gnu.org/software/gettext/manual/gettext.html#Header-Entry
	// Reference: http://www.gnu.org/software/gettext/manual/html_node/Header-Entry.html

	//Hard code some header entries for now in the following format
	/*
	# {ProjectName} {LanguageName} translation
	# {CopyrightNotice}
	#
	msgid ""
	msgstr ""
	"Project-Id-Version: {ProjectName}\n"
	"POT-Creation-Date: 2014-1-31 04:16+0000\n"
	"PO-Revision-Date: 2014-1-31 04:16+0000\n"
	"Language-Team: \n"
	"Language: {LanguageCode}\n"
	"MIME-Version: 1.0\n"
	"Content-Type: text/plain; charset=UTF-8\n"
	"Content-Transfer-Encoding: 8bit\n"
	"Plural-Forms: nplurals=2; plural=(n != 1);\n"
	*/

	Header.Clear();

	Header.SetEntryValue( TEXT("Project-Id-Version"), *GetProjectName() );

	// Setup header entries
	Header.UpdateTimeStamp();
	Header.SetEntryValue( TEXT("Language-Team"), TEXT("") );
	Header.SetEntryValue( TEXT("Language"), Language.GetLanguageCode() );
	Header.SetEntryValue( TEXT("MIME-Version"), TEXT("1.0") );
	Header.SetEntryValue( TEXT("Content-Type"), TEXT("text/plain; charset=UTF-8") );
	Header.SetEntryValue( TEXT("Content-Transfer-Encoding"), TEXT("8bit") );
	Header.SetEntryValue( TEXT("Plural-Forms"), Language.GetPluralForms() );

	Header.Comments.Add( FString::Printf(TEXT("%s %s translation."), *GetProjectName(), *Language.EnglishName() ) );
	if (!InCopyrightNotice.IsEmpty())
	{
		Header.Comments.Add(InCopyrightNotice);
	}
	Header.Comments.Add( FString(TEXT("")) );
}

void FPortableObjectFormatDOM::SetHeaderValue(const FString& EntryKey, const FString& EntryValue)
{
	Header.SetEntryValue(EntryKey, EntryValue);
}

bool FPortableObjectFormatDOM::SetLanguage( const FString& LanguageCode, const FString& LangPluralForms )
{
	FPortableObjectCulture NewLang(	LanguageCode, LangPluralForms );
	if( NewLang.IsValid() )
	{
		Language = NewLang;
		return true;
	}
	return false;
}

bool FPortableObjectFormatDOM::AddEntry( const TSharedRef< FPortableObjectEntry> LocEntry )
{
	TSharedPtr<FPortableObjectEntry> ExistingEntry = FindEntry( LocEntry );
	if( ExistingEntry.IsValid()  )
	{
		ExistingEntry->AddReferences( LocEntry->ReferenceComments );
		ExistingEntry->AddExtractedComments( LocEntry->ExtractedComments );

		// Checks for a situation that we don't know how to handle yet.  ex. What happens when we match all the IDs for an entry but another param differs?
		check( LocEntry->TranslatorComments == ExistingEntry->TranslatorComments );
	}
	else
	{
		Entries.Add( *LocEntry, LocEntry );
	}

	return true;

}

TSharedPtr<FPortableObjectEntry> FPortableObjectFormatDOM::FindEntry( const TSharedRef<const FPortableObjectEntry> LocEntry ) const
{
	return Entries.FindRef(*LocEntry);
}

TSharedPtr<FPortableObjectEntry> FPortableObjectFormatDOM::FindEntry( const FString& MsgId, const FString& MsgIdPlural, const FString& MsgCtxt ) const
{
	return Entries.FindRef(FPortableObjectEntryKey(MsgId, MsgIdPlural, MsgCtxt));
}

void FPortableObjectFormatDOM::SortEntries()
{
	// Sort keys.
	for( const auto& EntryPair : Entries )
	{
		EntryPair.Value->ReferenceComments.Sort();
	}

	// Sort by reference (source location) then namespace, then keys, then source text.
	auto SortingPredicate = [](const TSharedPtr<FPortableObjectEntry>& A, const TSharedPtr<FPortableObjectEntry>& B) -> bool
	{
		// compare reference comment
		const int32 ReferenceCommentCount = FMath::Max(A->ReferenceComments.Num(), B->ReferenceComments.Num());
		for (int32 index = 0; index < ReferenceCommentCount; ++index)
		{
			// If A has no more comments, it is before B.
			if (!A->ReferenceComments.IsValidIndex(index) && B->ReferenceComments.IsValidIndex(index))
			{
				return true;
			}
			// If B has no more comments, it is before A.
			if (A->ReferenceComments.IsValidIndex(index) && !B->ReferenceComments.IsValidIndex(index))
			{
				return false;
			}

			check(A->ReferenceComments.IsValidIndex(index) && B->ReferenceComments.IsValidIndex(index));

			// If A's reference is lexicographically less, it is before B.
			if (A->ReferenceComments[index] < B->ReferenceComments[index])
			{
				return true;
			}
			// If B's comment is lexicographically less, it is before A.
			if (A->ReferenceComments[index] > B->ReferenceComments[index])
			{
				return false;
			}
		}

		// Compare namespace
		if (A->MsgCtxt < B->MsgCtxt)
		{
			return true;
		}
		else if (A->MsgCtxt > B->MsgCtxt)
		{
			return false;
		}

		// Compare keys
		const int32 ExtractedCommentCount = FMath::Max(A->ExtractedComments.Num(), B->ExtractedComments.Num());
		for (int32 i = 0; i < ExtractedCommentCount; ++i)
		{
			// If A has no more comments, it is before B.
			if (!A->ExtractedComments.IsValidIndex(i) && B->ExtractedComments.IsValidIndex(i))
			{
				return true;
			}
			// If B has no more comments, it is before A.
			if (A->ExtractedComments.IsValidIndex(i) && !B->ExtractedComments.IsValidIndex(i))
			{
				return false;
			}

			check(A->ExtractedComments.IsValidIndex(i) && B->ExtractedComments.IsValidIndex(i));

			// If A's comment is lexicographically less, it is before B.
			if (A->ExtractedComments[i] < B->ExtractedComments[i])
			{
				return true;
			}
			// If B's comment is lexicographically less, it is before A.
			if (A->ExtractedComments[i] > B->ExtractedComments[i])
			{
				return false;
			}
		}

		// Compare source string
		if (A->MsgId < B->MsgId)
		{
			return true;
		}
		else if (A->MsgId > B->MsgId)
		{
			return false;
		}

		return A.Get() < B.Get();
	};
	Entries.ValueSort(SortingPredicate);
}

void FPortableObjectEntry::AddExtractedComment( const FString& InComment )
{
	if (!InComment.IsEmpty())
	{
		ExtractedComments.AddUnique(InComment);
	}
}

void FPortableObjectEntry::AddExtractedComments( const TArray<FString>& InComments )
{
	for( const FString& ExtractedComment : InComments )
	{
		AddExtractedComment( ExtractedComment );
	}
}

void FPortableObjectEntry::AddTranslatorComment(const FString& InComment)
{
	if (!InComment.IsEmpty())
	{
		TranslatorComments.AddUnique(InComment);
	}
}

void FPortableObjectEntry::AddTranslatorComments(const TArray<FString>& InComments)
{
	for (const FString& TranslatorComment : InComments)
	{
		AddTranslatorComment(TranslatorComment);
	}
}

void FPortableObjectEntry::AddReference( const FString& InReference )
{
	if (!InReference.IsEmpty())
	{
		ReferenceComments.AddUnique(InReference);
	}
}

void FPortableObjectEntry::AddReferences( const TArray<FString>& InReferences )
{
	for( const FString& Reference : InReferences )
	{
		AddReference( Reference );
	}
}

FString FPortableObjectEntry::ToString() const
{
	check( !MsgId.IsEmpty() );

	FString Result;

	for( const FString& TranslatorComment : TranslatorComments )
	{
		Result += FString::Printf( TEXT("# %s%s"), *TranslatorComment, NewLineDelimiter );
	}

	for( const FString& Comment : ExtractedComments )
	{
		if( !Comment.IsEmpty() )
		{
			Result += FString::Printf( TEXT("#. %s%s"), *Comment, NewLineDelimiter );
		}
		else
		{
			Result += FString::Printf( TEXT("#.%s"), NewLineDelimiter );
		}
	}

	for( const FString& ReferenceComment : ReferenceComments )
	{
		Result += FString::Printf( TEXT("#: %s%s"), *ReferenceComment, NewLineDelimiter );
	}

	if( !MsgCtxt.IsEmpty() )
	{
		Result += FString::Printf(TEXT("msgctxt \"%s\"%s"), *MsgCtxt, NewLineDelimiter);
	}

	Result += FString::Printf(TEXT("msgid \"%s\"%s"), *MsgId, NewLineDelimiter);

	if( MsgStr.Num() == 0)
	{
		Result += FString::Printf(TEXT("msgstr \"\"%s"), NewLineDelimiter);
	}
	else if( MsgStr.Num() == 1 )
	{
		Result += FString::Printf(TEXT("msgstr \"%s\"%s"), *MsgStr[0], NewLineDelimiter);
	}
	else
	{
		for( int32 Idx = 0; Idx < MsgStr.Num(); ++Idx )
		{
			Result += FString::Printf(TEXT("msgstr[%d] \"%s\"%s"), Idx, *MsgStr[Idx], NewLineDelimiter);
		}
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE
