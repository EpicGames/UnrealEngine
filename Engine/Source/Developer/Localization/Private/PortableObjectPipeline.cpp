// Copyright Epic Games, Inc. All Rights Reserved.

#include "PortableObjectPipeline.h"

#include "Async/ParallelFor.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Serialization/JsonInternationalizationMetadataSerializer.h"
#include "PortableObjectFormatDOM.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "LocalizationFileUtil.h"
#include "LocTextHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PortableObjectPipeline)

DEFINE_LOG_CATEGORY_STATIC(LogPortableObjectPipeline, Log, All);

namespace
{
	struct FLocKeyPair
	{
	public:
		FLocKeyPair(FLocKey InFirst, FLocKey InSecond)
			: First(MoveTemp(InFirst))
			, Second(MoveTemp(InSecond))
		{
		}

		FORCEINLINE bool operator==(const FLocKeyPair& Other) const
		{
			return First == Other.First && Second == Other.Second;
		}

		FORCEINLINE bool operator!=(const FLocKeyPair& Other) const
		{
			return First != Other.First || Second != Other.Second;
		}

		friend inline uint32 GetTypeHash(const FLocKeyPair& Id)
		{
			return HashCombine(GetTypeHash(Id.First), GetTypeHash(Id.Second));
		}

		FLocKey First;
		FLocKey Second;
	};
	typedef TMultiMap<FLocKeyPair, FLocKeyPair> FLocKeyPairMultiMap;

	struct FCollapsedData
	{
		/** Mapping between a collapsed namespace (First) and key (Second), to an expanded namespace (First) and key (Second) */
		FLocKeyPairMultiMap CollapsedNSKeyToExpandedNSKey;

		/** Mapping between a collapsed namespace (First) and source string/native translation (Second), to an expanded namespace (First) and key (Second) */
		FLocKeyPairMultiMap CollapsedNSSourceStringToExpandedNSKey;
	};

	struct FLocComments
	{
		FLocComments(TArray<FString> InExtracted, TArray<FString> InTranslator)
			: ExtractedComments(MoveTemp(InExtracted))
			, TranslatorComments(MoveTemp(InTranslator))
		{
		}

		TArray<FString> ExtractedComments;
		TArray<FString> TranslatorComments;
	};

	void BuildCollapsedManifest(FLocTextHelper& InLocTextHelper, const ELocalizedTextCollapseMode InTextCollapseMode, FCollapsedData& OutCollapsedData, TSharedPtr<FInternationalizationManifest>& OutPlatformAgnosticManifest, TMap<FName, TSharedRef<FInternationalizationManifest>>& OutPerPlatformManifests)
	{
		// Always add the split platforms so that they generate an empty manifest if there are no entries for that platform in the platform agnostic manifest
		OutPlatformAgnosticManifest = MakeShared<FInternationalizationManifest>();
		for (const FString& SplitPlatformName : InLocTextHelper.GetPlatformsToSplit())
		{
			OutPerPlatformManifests.Add(*SplitPlatformName, MakeShared<FInternationalizationManifest>());
		}

		InLocTextHelper.EnumerateSourceTexts([&](TSharedRef<FManifestEntry> InManifestEntry) -> bool
		{
			const FLocKey CollapsedNamespace = InTextCollapseMode == ELocalizedTextCollapseMode::IdenticalPackageIdTextIdAndSource ? InManifestEntry->Namespace : TextNamespaceUtil::StripPackageNamespace(InManifestEntry->Namespace.GetString());

			for (const FManifestContext& Context : InManifestEntry->Contexts)
			{
				bool bAddedContext = false;

				TSharedPtr<FInternationalizationManifest> ManifestToUpdate = OutPlatformAgnosticManifest;
				if (!Context.PlatformName.IsNone())
				{
					if (TSharedRef<FInternationalizationManifest>* PerPlatformManifest = OutPerPlatformManifests.Find(Context.PlatformName))
					{
						ManifestToUpdate = *PerPlatformManifest;
					}
				}
				check(ManifestToUpdate.IsValid());

				// Check if the entry already exists in the manifest
				TSharedPtr<FManifestEntry> ExistingEntry = ManifestToUpdate->FindEntryByContext(CollapsedNamespace, Context);
				if (ExistingEntry.IsValid())
				{
					if (InManifestEntry->Source.IsExactMatch(ExistingEntry->Source))
					{
						bAddedContext = true;
					}
					else
					{
						// Grab the source location of the conflicting context
						const FManifestContext* ConflictingContext = ExistingEntry->FindContext(Context.Key, Context.KeyMetadataObj);

						const FString Message = FLocTextHelper::SanitizeLogOutput(
							FString::Printf(TEXT("Found previously entered localized string: %s [%s] %s %s=\"%s\" %s. It was previously \"%s\" %s in %s."),
								*Context.SourceLocation,
								*CollapsedNamespace.GetString(),
								*Context.Key.GetString(),
								*FJsonInternationalizationMetaDataSerializer::MetadataToString(Context.KeyMetadataObj),
								*InManifestEntry->Source.Text,
								*FJsonInternationalizationMetaDataSerializer::MetadataToString(InManifestEntry->Source.MetadataObj),
								*ExistingEntry->Source.Text,
								*FJsonInternationalizationMetaDataSerializer::MetadataToString(ExistingEntry->Source.MetadataObj),
								*ConflictingContext->SourceLocation
							)
						);
						UE_LOGF(LogPortableObjectPipeline, Warning, "%ls", *Message);

						InLocTextHelper.AddConflict(CollapsedNamespace, Context.Key, Context.KeyMetadataObj, InManifestEntry->Source, Context.SourceLocation);
						InLocTextHelper.AddConflict(CollapsedNamespace, Context.Key, Context.KeyMetadataObj, ExistingEntry->Source, ConflictingContext->SourceLocation);
					}
				}
				else
				{
					if (ManifestToUpdate->AddSource(CollapsedNamespace, InManifestEntry->Source, Context))
					{
						bAddedContext = true;
					}
					else
					{
						UE_LOGF(LogPortableObjectPipeline, Error, "Could not process localized string: %ls [%ls] %ls=\"%ls\" %ls.",
							*Context.SourceLocation,
							*CollapsedNamespace.GetString(),
							*Context.Key.GetString(),
							*InManifestEntry->Source.Text,
							*FJsonInternationalizationMetaDataSerializer::MetadataToString(InManifestEntry->Source.MetadataObj)
						);
					}
				}

				if (bAddedContext)
				{
					// Add this collapsed namespace/key pair to our mapping so we can expand it again during import
					OutCollapsedData.CollapsedNSKeyToExpandedNSKey.AddUnique(FLocKeyPair(CollapsedNamespace, Context.Key), FLocKeyPair(InManifestEntry->Namespace, Context.Key));

					// Add this collapsed namespace/source string pair to our mapping so we expand it again during import (also map it against any native "translation" as that's what foreign imports will use as their source for translations)
					if (!Context.KeyMetadataObj.IsValid())
					{
						OutCollapsedData.CollapsedNSSourceStringToExpandedNSKey.AddUnique(FLocKeyPair(CollapsedNamespace, InManifestEntry->Source.Text), FLocKeyPair(InManifestEntry->Namespace, Context.Key));

						if (InLocTextHelper.HasNativeArchive())
						{
							TSharedPtr<FArchiveEntry> NativeTranslation = InLocTextHelper.FindTranslation(InLocTextHelper.GetNativeCulture(), InManifestEntry->Namespace, Context.Key, nullptr);
							if (NativeTranslation.IsValid() && !NativeTranslation->Translation.Text.Equals(InManifestEntry->Source.Text))
							{
								OutCollapsedData.CollapsedNSSourceStringToExpandedNSKey.AddUnique(FLocKeyPair(CollapsedNamespace, NativeTranslation->Translation.Text), FLocKeyPair(InManifestEntry->Namespace, Context.Key));
							}
						}
					}
				}
			}

			return true; // continue enumeration
		}, true);
	}

	TMap<FPortableObjectEntryKey, FLocComments> ExtractPreservedPOComments(const FPortableObjectFormatDOM& InPortableObject)
	{
		TMap<FPortableObjectEntryKey, FLocComments> POEntryToCommentMap;
		for (auto EntryPairIterator = InPortableObject.GetEntriesIterator(); EntryPairIterator; ++EntryPairIterator)
		{
			const TSharedPtr< FPortableObjectEntry >& Entry = EntryPairIterator->Value;

			// Preserve only non-procedurally generated extracted comments.
			const TArray<FString> CommentsToPreserve = Entry->ExtractedComments.FilterByPredicate([](const FString& ExtractedComment) -> bool
			{
				return !ExtractedComment.StartsWith(TEXT("Key:"), ESearchCase::CaseSensitive) && !ExtractedComment.StartsWith(TEXT("SourceLocation:"), ESearchCase::CaseSensitive) && !ExtractedComment.StartsWith(TEXT("InfoMetaData:"), ESearchCase::CaseSensitive) && !ExtractedComment.StartsWith(TEXT("DevNotes:"), ESearchCase::CaseSensitive);
			});

			if (CommentsToPreserve.Num() || Entry->TranslatorComments.Num())
			{
				POEntryToCommentMap.Add(FPortableObjectEntryKey(Entry->MsgId, Entry->MsgIdPlural, Entry->MsgCtxt), FLocComments(CommentsToPreserve, Entry->TranslatorComments));
			}
		}
		return POEntryToCommentMap;
	}

	bool LoadPOFile(const FString& POFilePath, FPortableObjectFormatDOM& OutPortableObject)
	{
		if (!FPaths::FileExists(POFilePath))
		{
			UE_LOGF(LogPortableObjectPipeline, Log, "Could not find file %ls", *POFilePath);
			return false;
		}

		FString POFileContents;
		if (!FFileHelper::LoadFileToString(POFileContents, *POFilePath))
		{
			UE_LOGF(LogPortableObjectPipeline, Error, "Failed to load file %ls.", *POFilePath);
			return false;
		}

		FText POErrorMsg;
		if (!OutPortableObject.FromString(POFileContents, &POErrorMsg))
		{
			UE_LOGF(LogPortableObjectPipeline, Error, "Failed to parse Portable Object file %ls: %ls", *POFilePath, *POErrorMsg.ToString());
			return false;
		}

		return true;
	}

	bool ImportPortableObject(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const FCollapsedData& InCollapsedData)
	{
		using namespace PortableObjectPipeline;

		if (!FPaths::FileExists(InPOFilePath))
		{
			UE_LOGF(LogPortableObjectPipeline, Log, "Could not find file %ls", *InPOFilePath);
			return true; // We don't fail on a missing file as the automation pipeline will always import before an export for a new language
		}

		FPortableObjectFormatDOM PortableObject;
		if (!LoadPOFile(InPOFilePath, PortableObject))
		{
			return false;
		}

		bool bModifiedArchive = false;
		{
			for (auto EntryPairIter = PortableObject.GetEntriesIterator(); EntryPairIter; ++EntryPairIter)
			{
				auto POEntry = EntryPairIter->Value;
				if (POEntry->MsgId.IsEmpty() || POEntry->MsgStr.Num() == 0 || POEntry->MsgStr[0].IsEmpty())
				{
					// We ignore the header entry or entries with no translation.
					continue;
				}

				// Some warning messages for data we don't process at the moment
				if (!POEntry->MsgIdPlural.IsEmpty() || POEntry->MsgStr.Num() > 1)
				{
					UE_LOGF(LogPortableObjectPipeline, Warning, "Portable Object entry has plural form we did not process.  File: %ls  MsgCtxt: %ls  MsgId: %ls", *InPOFilePath, *POEntry->MsgCtxt, *POEntry->MsgId);
				}

				FString SourceText;
				FString Translation;
				TArray<FLocKeyPair> NamespacesAndKeys; // Namespace (First) and Key (Second)
				{
					FString ParsedNamespace;
					FString ParsedKey;
					ParseBasicPOFileEntry(*POEntry, ParsedNamespace, ParsedKey, SourceText, Translation, InTextCollapseMode, InPOFormat);

					if (ParsedKey.IsEmpty())
					{
						// Legacy non-keyed PO entry - need to look-up the expanded namespace key/pairs via the namespace and source string
						InCollapsedData.CollapsedNSSourceStringToExpandedNSKey.MultiFind(FLocKeyPair(ParsedNamespace, SourceText), NamespacesAndKeys);
					}
					else
					{
						// Keyed PO entry - need to look-up the expanded namespace/key pairs via the namespace and key
						InCollapsedData.CollapsedNSKeyToExpandedNSKey.MultiFind(FLocKeyPair(ParsedNamespace, ParsedKey), NamespacesAndKeys);
					}
				}

				if (NamespacesAndKeys.Num() == 0)
				{
					UE_LOGF(LogPortableObjectPipeline, Log, "Could not import PO entry as it did not map to any known entries in the collapsed manifest data.  File: %ls  MsgCtxt: %ls  MsgId: %ls", *InPOFilePath, *POEntry->MsgCtxt, *POEntry->MsgId);
					continue;
				}

				for (const FLocKeyPair& NamespaceAndKey : NamespacesAndKeys)
				{
					// Alias for convenience of reading
					const FLocKey& Namespace = NamespaceAndKey.First;
					const FLocKey& Key = NamespaceAndKey.Second;

					// Get key metadata from the manifest, using the namespace and key.
					TSharedPtr<FManifestEntry> ManifestEntry = InLocTextHelper.FindSourceText(Namespace, Key);
					const FManifestContext* ItemContext = ManifestEntry ? ManifestEntry->FindContextByKey(Key) : nullptr;

					//@TODO: Take into account optional entries and entries that differ by keymetadata.  Ex. Each optional entry needs a unique msgCtxt

					// Not all formats contain the source string, so if the source is empty then 
					// we'll assume the translation was made against the most up-to-date source
					// Note: This sets the source text for all future iterations of this loop
					if (SourceText.IsEmpty())
					{
						// Find the correct translation based upon the native source text
						FLocItem ExportedSource;
						FLocItem ExportedTranslation;
						InLocTextHelper.GetExportText(InCulture, Namespace, Key, ItemContext ? ItemContext->KeyMetadataObj : nullptr, ELocTextExportSourceMethod::NativeText, ManifestEntry ? ManifestEntry->Source : FLocItem(), ExportedSource, ExportedTranslation);
						SourceText = ExportedSource.Text;
					}

					// Attempt to import the new text (if required)
					const TSharedPtr<FArchiveEntry> FoundEntry = InLocTextHelper.FindTranslation(InCulture, Namespace, Key, ItemContext ? ItemContext->KeyMetadataObj : nullptr);
					if (!FoundEntry.IsValid() || !FoundEntry->Source.Text.Equals(SourceText, ESearchCase::CaseSensitive) || !FoundEntry->Translation.Text.Equals(Translation, ESearchCase::CaseSensitive))
					{
						if (InLocTextHelper.ImportTranslation(InCulture, Namespace, Key, ItemContext ? ItemContext->KeyMetadataObj : nullptr, FLocItem(SourceText), FLocItem(Translation), ItemContext && ItemContext->bIsOptional))
						{
							bModifiedArchive = true;
						}
					}
				}
			}
		}

		UE_LOGF(LogPortableObjectPipeline, Log, "Imported file %ls", *InPOFilePath);

		if (bModifiedArchive)
		{
			// Trim any dead entries out of the archive
			InLocTextHelper.TrimArchive(InCulture);

			FText SaveError;
			if (!InLocTextHelper.SaveArchive(InCulture, &SaveError))
			{
				UE_LOGF(LogPortableObjectPipeline, Error, "%ls", *SaveError.ToString());
				return false;
			}
		}

		return true;
	}

	bool ExportPortableObject(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, TSharedRef<FInternationalizationManifest> InCollapsedManifest, const FCollapsedData& InCollapsedData, const bool bShouldPersistComments)
	{
		using namespace PortableObjectPipeline;

		FPortableObjectFormatDOM NewPortableObject;

		FString LocLang;
		if (!NewPortableObject.SetLanguage(InCulture))
		{
			UE_LOGF(LogPortableObjectPipeline, Error, "Skipping export of culture %ls because it is not recognized PO language.", *InCulture);
			return false;
		}

		NewPortableObject.SetProjectName(FPaths::GetBaseFilename(InPOFilePath));
		NewPortableObject.CreateNewHeader(InLocTextHelper.GetCopyrightNotice());
		UpdatePOFileHeaderForSettings(NewPortableObject, InTextCollapseMode, InPOFormat);

		// Add each manifest entry to the PO file
		for (FManifestEntryByStringContainer::TConstIterator ManifestIterator = InCollapsedManifest->GetEntriesBySourceTextIterator(); ManifestIterator; ++ManifestIterator)
		{
			const TSharedRef<FManifestEntry> ManifestEntry = ManifestIterator.Value();

			// For each context, we may need to create a different or even multiple PO entries.
			for (const FManifestContext& Context : ManifestEntry->Contexts)
			{
				TSharedRef<FPortableObjectEntry> PoEntry = MakeShareable(new FPortableObjectEntry());

				// For export we just use the first expanded namespace/key pair to find the current translation (they should all be identical due to how the import works)
				const FLocKeyPair& ExportNamespaceKeyPair = InCollapsedData.CollapsedNSKeyToExpandedNSKey.FindChecked(FLocKeyPair(ManifestEntry->Namespace, Context.Key));

				// Find the correct translation based upon the native source text
				FLocItem ExportedSource;
				FLocItem ExportedTranslation;
				InLocTextHelper.GetExportText(InCulture, ExportNamespaceKeyPair.First, ExportNamespaceKeyPair.Second, Context.KeyMetadataObj, ELocTextExportSourceMethod::NativeText, ManifestEntry->Source, ExportedSource, ExportedTranslation);

				PopulateBasicPOFileEntry(*PoEntry, ManifestEntry->Namespace.GetString(), Context.Key.GetString(), Context.KeyMetadataObj, ExportedSource.Text, ExportedTranslation.Text, InTextCollapseMode, InPOFormat);

				//@TODO: We support additional metadata entries that can be translated.  How do those fit in the PO file format?  Ex: isMature
				PoEntry->AddReference(Context.SourceLocation); // Source location.

				PoEntry->AddExtractedComment(GetConditionedKeyForExtractedComment(Context.Key.GetString())); // "Notes from Programmer" in the form of the Key.
				PoEntry->AddExtractedComment(GetConditionedReferenceForExtractedComment(Context.SourceLocation)); // "Notes from Programmer" in the form of the Source Location, since this comes in handy too and OneSky doesn't properly show references, only comments.
#if WITH_EDITORONLY_DATA
				if (!Context.DevNotes.IsEmpty())
				{
					PoEntry->AddExtractedComment(GetConditionedDevNotesForExtractedComment(Context.DevNotes)); // "Notes from Programmer" in the form of the notes entered by the developer.
					}
#endif

				TArray<FString> InfoMetaDataStrings;
				if (Context.InfoMetadataObj.IsValid())
				{
					for (auto InfoMetaDataPair : Context.InfoMetadataObj->Values)
					{
						const FString KeyName = InfoMetaDataPair.Key;
						const TSharedPtr<FLocMetadataValue> Value = InfoMetaDataPair.Value;
						InfoMetaDataStrings.Add(GetConditionedInfoMetaDataForExtractedComment(KeyName, ConditionArchiveStrForPO(Value->ToString())));
					}
				}
				if (InfoMetaDataStrings.Num())
				{
					PoEntry->AddExtractedComments(InfoMetaDataStrings);
				}

				NewPortableObject.AddEntry(PoEntry);
			}
		}

		// Persist comments if requested.
		if (bShouldPersistComments)
		{
			// Preserve comments from the specified file now
			TMap<FPortableObjectEntryKey, FLocComments> POEntryToCommentMap;
			{
				FPortableObjectFormatDOM ExistingPortableObject;
				if (LoadPOFile(InPOFilePath, ExistingPortableObject))
				{
					POEntryToCommentMap = ExtractPreservedPOComments(ExistingPortableObject);
				}
			}

			// Persist the comments into the new portable object we're going to be saving.
			for (const auto& Pair : POEntryToCommentMap)
			{
				const TSharedPtr<FPortableObjectEntry> FoundEntry = NewPortableObject.FindEntry(Pair.Key.MsgId, Pair.Key.MsgIdPlural, Pair.Key.MsgCtxt);
				if (FoundEntry.IsValid())
				{
					FoundEntry->AddExtractedComments(Pair.Value.ExtractedComments);
					FoundEntry->AddTranslatorComments(Pair.Value.TranslatorComments);
				}
			}
		}

		NewPortableObject.SortEntries();

		const bool bPOFileSaved = UE::Localization::FileUtil::WriteFileIfModified(InPOFilePath,
			[&NewPortableObject](const FString& Filename)
			{
				//@TODO We force UTF8 at the moment but we want this to be based on the format found in the header info.
				return FFileHelper::SaveStringToFile(NewPortableObject.ToString(), *Filename, FFileHelper::EEncodingOptions::ForceUTF8);
			}, &UE::Localization::FileUtil::PortableObjectHash, InLocTextHelper.GetLocFileNotifies());

		if (!bPOFileSaved)
		{
			UE_LOGF(LogPortableObjectPipeline, Error, "Could not write file %ls", *InPOFilePath);
			return false;
		}

		UE_LOGF(LogPortableObjectPipeline, Log, "Exported file %ls", *InPOFilePath);

		return true;
	}
}

bool PortableObjectPipeline::Import(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat)
{
	// This function only works when not splitting per-platform data
	if (InLocTextHelper.ShouldSplitPlatformData())
	{
		UE_LOGF(LogPortableObjectPipeline, Error, "PortableObjectPipeline::Import may only be used when not splitting platform data.");
		return false;
	}

	// Build the collapsed manifest data needed to import
	FCollapsedData CollapsedData;
	TSharedPtr<FInternationalizationManifest> PlatformAgnosticManifest;
	TMap<FName, TSharedRef<FInternationalizationManifest>> PerPlatformManifests;
	BuildCollapsedManifest(InLocTextHelper, InTextCollapseMode, CollapsedData, PlatformAgnosticManifest, PerPlatformManifests);

	return ImportPortableObject(InLocTextHelper, InCulture, InPOFilePath, InTextCollapseMode, InPOFormat, CollapsedData);
}

bool PortableObjectPipeline::ImportAll(FLocTextHelper& InLocTextHelper, const FString& InPOCultureRootPath, const FString& InPOFilename, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bUseCultureDirectory)
{
	// We may only have a single culture if using this setting
	const bool bSingleCultureMode = !bUseCultureDirectory;
	const TArray<FString> Cultures = InLocTextHelper.GetAllCultures(bSingleCultureMode);
	if (bSingleCultureMode && (Cultures.Num() != 1 || InLocTextHelper.ShouldSplitPlatformData()))
	{
		UE_LOGF(LogPortableObjectPipeline, Error, "bUseCultureDirectory may only be used with a single culture when not splitting platform data.");
		return false;
	}

	// Build the collapsed manifest data needed to import
	FCollapsedData CollapsedData;
	TSharedPtr<FInternationalizationManifest> PlatformAgnosticManifest;
	TMap<FName, TSharedRef<FInternationalizationManifest>> PerPlatformManifests;
	BuildCollapsedManifest(InLocTextHelper, InTextCollapseMode, CollapsedData, PlatformAgnosticManifest, PerPlatformManifests);

	TSharedPtr<ILocFileNotifies> LocFileNotifies = InLocTextHelper.GetLocFileNotifies();

	if (LocFileNotifies)
	{
		LocFileNotifies->BeginParallelTasks();
	}

	// Process the desired cultures
	std::atomic<bool> bSuccess = true;
	ParallelFor(TEXT("PortableObjectPipeline::ImportAll"), Cultures.Num(), 1, [&](int32 Index)
	{
		const FString& CultureName = Cultures[Index];

		auto ImportSinglePortableObject = [&](const FName InPlatformName) -> bool
		{
			// Which path should we use for the PO?
			FString POFilePath;
			if (bUseCultureDirectory)
			{
				// Platforms splits are only supported when exporting all cultures (see the error at the start of this function)
				if (InPlatformName.IsNone())
				{
					POFilePath = InPOCultureRootPath / CultureName / InPOFilename;
				}
				else
				{
					POFilePath = InPOCultureRootPath / FPaths::GetPlatformLocalizationFolderName() / InPlatformName.ToString() / CultureName / InPOFilename;
				}
			}
			else
			{
				POFilePath = InPOCultureRootPath / InPOFilename;
			}

			return ImportPortableObject(InLocTextHelper, CultureName, POFilePath, InTextCollapseMode, InPOFormat, CollapsedData);
		};

		if (!ImportSinglePortableObject(FName()))
		{
			bSuccess = false;
		}
		for (const auto& PerPlatformManifestPair : PerPlatformManifests)
		{
			if (!ImportSinglePortableObject(PerPlatformManifestPair.Key))
			{
				bSuccess = false;
			}
		}
	});

	if (LocFileNotifies)
	{
		LocFileNotifies->EndParallelTasks();
	}

	return bSuccess;
}

bool PortableObjectPipeline::Export(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bShouldPersistComments)
{
	// This function only works when not splitting per-platform data
	if (InLocTextHelper.ShouldSplitPlatformData())
	{
		UE_LOGF(LogPortableObjectPipeline, Error, "PortableObjectPipeline::Export may only be used when not splitting platform data.");
		return false;
	}

	// Build the collapsed manifest data needed to export
	FCollapsedData CollapsedData;
	TSharedPtr<FInternationalizationManifest> PlatformAgnosticManifest;
	TMap<FName, TSharedRef<FInternationalizationManifest>> PerPlatformManifests;
	BuildCollapsedManifest(InLocTextHelper, InTextCollapseMode, CollapsedData, PlatformAgnosticManifest, PerPlatformManifests);

	return ExportPortableObject(InLocTextHelper, InCulture, InPOFilePath, InTextCollapseMode, InPOFormat, PlatformAgnosticManifest.ToSharedRef(), CollapsedData, bShouldPersistComments);
}

bool PortableObjectPipeline::ExportAll(FLocTextHelper& InLocTextHelper, const FString& InPOCultureRootPath, const FString& InPOFilename, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bShouldPersistComments, const bool bUseCultureDirectory)
{
	// We may only have a single culture if using this setting
	const bool bSingleCultureMode = !bUseCultureDirectory;
	const TArray<FString> Cultures = InLocTextHelper.GetAllCultures(bSingleCultureMode);
	if (bSingleCultureMode && (Cultures.Num() != 1 || InLocTextHelper.ShouldSplitPlatformData()))
	{
		UE_LOGF(LogPortableObjectPipeline, Error, "bUseCultureDirectory may only be used with a single culture when not splitting platform data.");
		return false;
	}

	// The 4.14 export mode was removed in 4.17
	if (InTextCollapseMode == ELocalizedTextCollapseMode::IdenticalPackageIdTextIdAndSource)
	{
		UE_LOGF(LogPortableObjectPipeline, Error, "The export mode 'ELocalizedTextCollapseMode::IdenticalPackageIdTextIdAndSource' is no longer supported (it was deprecated in 4.15 and removed in 4.17). Please use 'ELocalizedTextCollapseMode::IdenticalTextIdAndSource' instead.");
		return false;
	}

	// Build the collapsed manifest data to export
	FCollapsedData CollapsedData;
	TSharedPtr<FInternationalizationManifest> PlatformAgnosticManifest;
	TMap<FName, TSharedRef<FInternationalizationManifest>> PerPlatformManifests;
	BuildCollapsedManifest(InLocTextHelper, InTextCollapseMode, CollapsedData, PlatformAgnosticManifest, PerPlatformManifests);

	TSharedPtr<ILocFileNotifies> LocFileNotifies = InLocTextHelper.GetLocFileNotifies();

	if (LocFileNotifies)
	{
		LocFileNotifies->BeginParallelTasks();
	}

	// Process the desired cultures
	std::atomic<bool> bSuccess = true;
	ParallelFor(TEXT("PortableObjectPipeline::ExportAll"), Cultures.Num(), 1, [&](int32 Index)
	{
		const FString& CultureName = Cultures[Index];

		auto ExportSinglePortableObject = [&](const TSharedRef<FInternationalizationManifest>& InCollapsedManifest, const FName InPlatformName) -> bool
		{
			// Which path should we use for the PO?
			FString POFilePath;
			if (bUseCultureDirectory)
			{
				// Platforms splits are only supported when exporting all cultures (see the error at the start of this function)
				if (InPlatformName.IsNone())
				{
					POFilePath = InPOCultureRootPath / CultureName / InPOFilename;
				}
				else
				{
					POFilePath = InPOCultureRootPath / FPaths::GetPlatformLocalizationFolderName() / InPlatformName.ToString() / CultureName / InPOFilename;
				}
			}
			else
			{
				POFilePath = InPOCultureRootPath / InPOFilename;
			}

			return ExportPortableObject(InLocTextHelper, CultureName, POFilePath, InTextCollapseMode, InPOFormat, InCollapsedManifest, CollapsedData, bShouldPersistComments);
		};

		if (!ExportSinglePortableObject(PlatformAgnosticManifest.ToSharedRef(), FName()))
		{
			bSuccess = false;
		}
		for (const auto& PerPlatformManifestPair : PerPlatformManifests)
		{
			if (!ExportSinglePortableObject(PerPlatformManifestPair.Value, PerPlatformManifestPair.Key))
			{
				bSuccess = false;
			}
		}
	});

	if (LocFileNotifies)
	{
		LocFileNotifies->EndParallelTasks();
	}

	return bSuccess;
}

FString PortableObjectPipeline::ConditionIdentityForPO(const FString& Namespace, const FString& Key, const TSharedPtr<FLocMetadataObject>& KeyMetaData, const ELocalizedTextCollapseMode InTextCollapseMode)
{
	auto EscapeMsgCtxtParticleInline = [](FString& InStr)
	{
		InStr.ReplaceInline(TEXT(","), TEXT("\\,"), ESearchCase::CaseSensitive);
	};

	FString EscapedNamespace = Namespace;
	EscapeMsgCtxtParticleInline(EscapedNamespace);

	FString EscapedKey = Key;
	EscapeMsgCtxtParticleInline(EscapedKey);

	const bool bAppendKey = InTextCollapseMode != ELocalizedTextCollapseMode::IdenticalNamespaceAndSource || KeyMetaData.IsValid();
	return ConditionArchiveStrForPO(bAppendKey ? FString::Printf(TEXT("%s,%s"), *EscapedNamespace, *EscapedKey) : EscapedNamespace);
}

void PortableObjectPipeline::ParseIdentityFromPO(const FString& InIdentity, FString& OutNamespace, FString& OutKey)
{
	auto UnescapeMsgCtxtParticleInline = [](FString& InStr)
	{
		InStr.ReplaceInline(TEXT("\\,"), TEXT(","), ESearchCase::CaseSensitive);
	};

	FString ConditionedIdentity = ConditionPOStringForArchive(InIdentity);

	// Find the unescaped comma that defines the breaking point between the namespace and the key
	int32 CommaIndex = INDEX_NONE;
	{
		bool bIsEscaped = false;
		for (int32 Index = 0; Index < ConditionedIdentity.Len(); ++Index)
		{
			if (bIsEscaped)
			{
				// No longer escaped, and skip this character
				bIsEscaped = false;
				continue;
			}

			if (ConditionedIdentity[Index] == TEXT(','))
			{
				// Found the unescaped comma
				CommaIndex = Index;
				break;
			}

			if (ConditionedIdentity[Index] == TEXT('\\'))
			{
				// Next character will be escaped
				bIsEscaped = true;
				continue;
			}
		}
	}

	if (CommaIndex == INDEX_NONE)
	{
		OutNamespace = MoveTemp(ConditionedIdentity);
		OutKey.Reset();
	}
	else
	{
		OutNamespace = ConditionedIdentity.Mid(0, CommaIndex);
		OutKey = ConditionedIdentity.Mid(CommaIndex + 1);
	}

	UnescapeMsgCtxtParticleInline(OutNamespace);
	UnescapeMsgCtxtParticleInline(OutKey);
}

FString PortableObjectPipeline::ConditionArchiveStrForPO(const FString& InStr)
{
	FString Result = InStr;
	Result.ReplaceInline(TEXT("\\"), TEXT("\\\\"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\""), TEXT("\\\""), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\r"), TEXT("\\r"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\n"), TEXT("\\n"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\t"), TEXT("\\t"), ESearchCase::CaseSensitive);
	return Result;
}

FString PortableObjectPipeline::ConditionPOStringForArchive(const FString& InStr)
{
	FString Result = InStr;
	Result.ReplaceInline(TEXT("\\t"), TEXT("\t"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\\n"), TEXT("\n"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\\r"), TEXT("\r"), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\\\""), TEXT("\""), ESearchCase::CaseSensitive);
	Result.ReplaceInline(TEXT("\\\\"), TEXT("\\"), ESearchCase::CaseSensitive);
	return Result;
}

FString PortableObjectPipeline::GetConditionedKeyForExtractedComment(const FString& Key)
{
	return FString::Printf(TEXT("Key:\t%s"), *Key);
}

FString PortableObjectPipeline::GetConditionedReferenceForExtractedComment(const FString& PORefString)
{
	return FString::Printf(TEXT("SourceLocation:\t%s"), *PORefString);
}

FString PortableObjectPipeline::GetConditionedDevNotesForExtractedComment(const FString& DevNotes)
{
	return FString::Printf(TEXT("DevNotes:\t%s"), *ConditionArchiveStrForPO(DevNotes));
}

FString PortableObjectPipeline::GetConditionedInfoMetaDataForExtractedComment(const FString& KeyName, const FString& ValueString)
{
	return FString::Printf(TEXT("InfoMetaData:\t\"%s\" : \"%s\""), *KeyName, *ValueString);
}

void PortableObjectPipeline::UpdatePOFileHeaderForSettings(FPortableObjectFormatDOM& PortableObject, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat)
{
	if (InTextCollapseMode == ELocalizedTextCollapseMode::IdenticalTextIdAndSource && InPOFormat == EPortableObjectFormat::Crowdin)
	{
		PortableObject.SetHeaderValue(TEXT("X-Crowdin-SourceKey"), TEXT("msgstr"));
	}
}

void PortableObjectPipeline::PopulateBasicPOFileEntry(FPortableObjectEntry& POEntry, const FString& InNamespace, const FString& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetaData, const FString& InSourceString, const FString& InTranslation, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat)
{
	if (InTextCollapseMode == ELocalizedTextCollapseMode::IdenticalTextIdAndSource && InPOFormat == EPortableObjectFormat::Crowdin)
	{
		POEntry.MsgCtxt.Reset();
		POEntry.MsgId = ConditionIdentityForPO(InNamespace, InKey, InKeyMetaData, InTextCollapseMode);
		POEntry.MsgStr.Add(ConditionArchiveStrForPO(InTranslation));
	}
	else
	{
		POEntry.MsgCtxt = ConditionIdentityForPO(InNamespace, InKey, InKeyMetaData, InTextCollapseMode);
		POEntry.MsgId = ConditionArchiveStrForPO(InSourceString);
		POEntry.MsgStr.Add(ConditionArchiveStrForPO(InTranslation));
	}
}

void PortableObjectPipeline::ParseBasicPOFileEntry(const FPortableObjectEntry& POEntry, FString& OutNamespace, FString& OutKey, FString& OutSourceString, FString& OutTranslation, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat)
{
	if (InTextCollapseMode == ELocalizedTextCollapseMode::IdenticalTextIdAndSource && InPOFormat == EPortableObjectFormat::Crowdin)
	{
		ParseIdentityFromPO(POEntry.MsgId, OutNamespace, OutKey);
		OutSourceString.Reset(); // This format doesn't contain the source string
		OutTranslation = ConditionPOStringForArchive(POEntry.MsgStr[0]);
	}
	else
	{
		ParseIdentityFromPO(POEntry.MsgCtxt, OutNamespace, OutKey);
		OutSourceString = ConditionPOStringForArchive(POEntry.MsgId);
		OutTranslation = ConditionPOStringForArchive(POEntry.MsgStr[0]);
	}
}

