// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonInternationalizationArchiveSerializer.h"

#include "LocTextHelper.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonInternationalizationMetadataSerializer.h"


DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationArchiveSerializer, Log, All);


const FString FJsonInternationalizationArchiveSerializer::TAG_FORMATVERSION = TEXT("FormatVersion");
const FString FJsonInternationalizationArchiveSerializer::TAG_NAMESPACE = TEXT("Namespace");
const FString FJsonInternationalizationArchiveSerializer::TAG_KEY = TEXT("Key");
const FString FJsonInternationalizationArchiveSerializer::TAG_CHILDREN = TEXT("Children");
const FString FJsonInternationalizationArchiveSerializer::TAG_SUBNAMESPACES = TEXT("Subnamespaces");
const FString FJsonInternationalizationArchiveSerializer::TAG_DEPRECATED_DEFAULTTEXT = TEXT("DefaultText");
const FString FJsonInternationalizationArchiveSerializer::TAG_DEPRECATED_TRANSLATEDTEXT = TEXT("TranslatedText");
const FString FJsonInternationalizationArchiveSerializer::TAG_OPTIONAL = TEXT("Optional");
const FString FJsonInternationalizationArchiveSerializer::TAG_SOURCE = TEXT("Source");
const FString FJsonInternationalizationArchiveSerializer::TAG_SOURCE_TEXT = TEXT("Text");
const FString FJsonInternationalizationArchiveSerializer::TAG_TRANSLATION = TEXT("Translation");
const FString FJsonInternationalizationArchiveSerializer::TAG_TRANSLATION_TEXT = FJsonInternationalizationArchiveSerializer::TAG_SOURCE_TEXT;
const FString FJsonInternationalizationArchiveSerializer::TAG_METADATA = TEXT("MetaData");
const FString FJsonInternationalizationArchiveSerializer::TAG_METADATA_KEY = TEXT("Key");
const FString FJsonInternationalizationArchiveSerializer::NAMESPACE_DELIMITER = TEXT(".");


bool FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromString(const FString& InStr, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive)
{
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Document = UE::Json::Parse(InStr);

	if (Document.HasError())
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to parse archive. %ls.", *Document.GetError().CreateMessage(InStr));
		return false;
	}

	TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(Document.GetValue());
	if (!RootObject.IsSet())
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to parse archive. No root object.");
		return false;
	}

	return DeserializeInternal(*RootObject, InArchive, InManifest, InNativeArchive);
}


bool FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromFile(const FString& InJsonFile, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive)
{
	// Read in file as string
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *InJsonFile))
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to load archive '%ls'.", *InJsonFile);
		return false;
	}

	// Grab the internal character array so we can do an insitu parse
	TArray<TCHAR> FileData = MoveTemp(FileContents.GetCharArray());
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Document = UE::Json::ParseInPlace(FileData);

	if (Document.HasError())
	{
		// Have to load the file again as the JSON contents was rewritten by the insitu parse
		ensure(FFileHelper::LoadFileToString(FileContents, *InJsonFile));
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to parse archive '%ls'. %ls.", *InJsonFile, *Document.GetError().CreateMessage(FileContents));
		return false;
	}

	TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(Document.GetValue());
	if (!RootObject.IsSet())
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to parse archive '%ls'. No root object.", *InJsonFile);
		return false;
	}

	return DeserializeInternal(*RootObject, InArchive, InManifest, InNativeArchive);
}


bool FJsonInternationalizationArchiveSerializer::SerializeArchiveToString( TSharedRef< const FInternationalizationArchive > InArchive, FString& Str )
{
	UE::Json::FDocument Document(rapidjson::kObjectType);

	if (SerializeInternal(InArchive, Document.GetObject(), Document.GetAllocator()))
	{
		Str = UE::Json::WritePretty(Document);
		return true;
	}

	return false;
}


bool FJsonInternationalizationArchiveSerializer::SerializeArchiveToFile(TSharedRef<const FInternationalizationArchive> InArchive, const FString& InJsonFile)
{
	FString JsonString;
	if (!SerializeArchiveToString(InArchive, JsonString))
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to serialize archive '%ls'.", *InJsonFile);
		return false;
	}

	// Save the JSON string (force Unicode for our manifest and archive files)
	// TODO: Switch to UTF-8 by default, unless the file on-disk is already UTF-16
	if (!FFileHelper::SaveStringToFile(JsonString, *InJsonFile, FFileHelper::EEncodingOptions::ForceUnicode))
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Error, "Failed to save archive '%ls'.", *InJsonFile);
		return false;
	}

	return true;
}


bool FJsonInternationalizationArchiveSerializer::DeserializeInternal(UE::Json::FConstObject InJsonObj, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive)
{
	if (TOptional<int32> FormatVersion = UE::Json::GetInt32Field(InJsonObj, *TAG_FORMATVERSION))
	{
		if (*FormatVersion > (int32)FInternationalizationArchive::EFormatVersion::Latest)
		{
			// Archive is too new to be loaded!
			return false;
		}

		InArchive->SetFormatVersion(static_cast<FInternationalizationArchive::EFormatVersion>(*FormatVersion));
	}
	else
	{
		InArchive->SetFormatVersion(FInternationalizationArchive::EFormatVersion::Initial);
	}

	if (InArchive->GetFormatVersion() < FInternationalizationArchive::EFormatVersion::AddedKeys && !InManifest.IsValid())
	{
		// Cannot load these archives without a manifest to key against
		return false;
	}

	if (JsonObjToArchive(InJsonObj, FString(), InArchive, InManifest, InNativeArchive))
	{
		// We've been upgraded to the latest format now...
		InArchive->SetFormatVersion(FInternationalizationArchive::EFormatVersion::Latest);
		return true;
	}

	return false;
}


bool FJsonInternationalizationArchiveSerializer::SerializeInternal(TSharedRef<const FInternationalizationArchive> InArchive, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator)
{
	TSharedPtr<FStructuredArchiveEntry> RootElement = MakeShared<FStructuredArchiveEntry>(FString());

	// Condition the data so that it exists in a structured hierarchy for easy population of the JSON object.
	GenerateStructuredData(InArchive, RootElement);

	SortStructuredData(RootElement);

	// Clear out anything that may be in the JSON object
	JsonObj.RemoveAllMembers();

	// Set format version.
	JsonObj.AddMember(UE::Json::MakeStringRef(TAG_FORMATVERSION), UE::Json::FValue((int32)InArchive->GetFormatVersion()), Allocator);

	// Setup the JSON object using the structured data created
	StructuredDataToJsonObj(RootElement, JsonObj, Allocator);

	return true;
}


bool FJsonInternationalizationArchiveSerializer::JsonObjToArchive(UE::Json::FConstObject InJsonObj, const FString& ParentNamespace, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive)
{
	FString AccumulatedNamespace = ParentNamespace;

	if (TOptional<FStringView> Namespace = UE::Json::GetStringField(InJsonObj, *TAG_NAMESPACE))
	{
		if (!AccumulatedNamespace.IsEmpty())
		{
			AccumulatedNamespace += NAMESPACE_DELIMITER;
		}
		AccumulatedNamespace += *Namespace;
	}
	else
	{
		UE_LOGF(LogInternationalizationArchiveSerializer, Warning, "Encountered an object with a missing namespace while converting to Internationalization archive.");
		return false;
	}

	// Process all the child objects
	if (TOptional<UE::Json::FConstArray> ChildrenArray = UE::Json::GetArrayField(InJsonObj, *TAG_CHILDREN))
	{
		for (const UE::Json::FValue& ChildEntry : *ChildrenArray)
		{
			if (!ChildEntry.IsObject())
			{
				return false;
			}

			UE::Json::FConstObject ChildJSONObject = ChildEntry.GetObject();

			FString SourceText;
			TSharedPtr< FLocMetadataObject > SourceMetadata;
			if (TOptional<UE::Json::FConstObject> SourceJSONObject = UE::Json::GetObjectField(ChildJSONObject, *TAG_SOURCE))
			{
				if (TOptional<FStringView> SourceTextValue = UE::Json::GetStringField(*SourceJSONObject, *TAG_SOURCE_TEXT))
				{
					SourceText = *SourceTextValue;

					// Source meta data is mixed in with the source text, we'll process metadata if the source json object has more than one entry
					if (SourceJSONObject->MemberCount() > 1)
					{
						// We load in the entire source object as metadata and just remove the source text.
						FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*SourceJSONObject, SourceMetadata);
						if (SourceMetadata)
						{
							SourceMetadata->Values.Remove(TAG_SOURCE_TEXT);
						}
					}
				}
				else
				{
					return false;
				}
			}
			else if (TOptional<FStringView> DefaultText = UE::Json::GetStringField(ChildJSONObject, *TAG_DEPRECATED_DEFAULTTEXT))
			{
				SourceText = *DefaultText;
			}
			else
			{
				return false;
			}

			FString TranslationText;
			TSharedPtr< FLocMetadataObject > TranslationMetadata;
			if (TOptional<UE::Json::FConstObject> TranslationJSONObject = UE::Json::GetObjectField(ChildJSONObject, *TAG_TRANSLATION))
			{
				if (TOptional<FStringView> TranslationTextValue = UE::Json::GetStringField(*TranslationJSONObject, *TAG_TRANSLATION_TEXT))
				{
					TranslationText = *TranslationTextValue;

					// Translation meta data is mixed in with the translation text, we'll process metadata if the source json object has more than one entry
					if (TranslationJSONObject->MemberCount() > 1)
					{
						// We load in the entire translation object as metadata and remove the translation text
						FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*TranslationJSONObject, TranslationMetadata);
						if (TranslationMetadata)
						{
							TranslationMetadata->Values.Remove(TAG_TRANSLATION_TEXT);
						}
					}
				}
				else if (TOptional<FStringView> TranslatedText = UE::Json::GetStringField(ChildJSONObject, *TAG_DEPRECATED_TRANSLATEDTEXT))
				{
					TranslationText = *TranslatedText;
				}
				else
				{
					return false;
				}
			}
			else
			{
				return false;
			}

			FLocItem Source(SourceText);
			Source.MetadataObj = SourceMetadata;

			FLocItem Translation(TranslationText);
			Translation.MetadataObj = TranslationMetadata;

			bool bIsOptional = false;
			if (TOptional<bool> IsOptionalValue = UE::Json::GetBoolField(ChildJSONObject, *TAG_OPTIONAL))
			{
				bIsOptional = *IsOptionalValue;
			}

			TArray<FLocKey> Keys;
			TSharedPtr< FLocMetadataObject > KeyMetadataNode;
			if (InArchive->GetFormatVersion() < FInternationalizationArchive::EFormatVersion::AddedKeys)
			{
				// We used to store the key meta-data as a top-level value, rather than within a "MetaData" object
				if (TOptional<UE::Json::FConstObject> MetaDataKeyJSONObject = UE::Json::GetObjectField(ChildJSONObject, *TAG_METADATA_KEY))
				{
					FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*MetaDataKeyJSONObject, KeyMetadataNode);
				}

				if (InManifest)
				{
					// We have no key in the archive data, so we must try and infer it from the manifest
					FLocTextHelper::FindKeysForLegacyTranslation(InManifest.ToSharedRef(), InNativeArchive, AccumulatedNamespace, SourceText, KeyMetadataNode, Keys);
				}
			}
			else
			{
				if (TOptional<FStringView> KeyValue = UE::Json::GetStringField(ChildJSONObject, *TAG_KEY))
				{
					Keys.Add(FString(*KeyValue));
				}

				if (TOptional<UE::Json::FConstObject> MetaDataJSONObject = UE::Json::GetObjectField(ChildJSONObject, *TAG_METADATA))
				{
					if (TOptional<UE::Json::FConstObject> MetaDataKeyJSONObject = UE::Json::GetObjectField(*MetaDataJSONObject, *TAG_METADATA_KEY))
					{
						FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*MetaDataKeyJSONObject, KeyMetadataNode);
					}
				}
			}

			for (const FLocKey& Key : Keys)
			{
				const bool bAddSuccessful = InArchive->AddEntry(AccumulatedNamespace, Key, Source, Translation, KeyMetadataNode, bIsOptional);
				if (!bAddSuccessful)
				{
					UE_LOGF(LogInternationalizationArchiveSerializer, Warning, "Could not add JSON entry to the Internationalization archive: Namespace:%ls Key:%ls DefaultText:%ls", *AccumulatedNamespace, *Key.GetString(), *SourceText);
				}
			}
		}
	}

	if (TOptional<UE::Json::FConstArray> SubnamespaceArray = UE::Json::GetArrayField(InJsonObj, *TAG_SUBNAMESPACES))
	{
		for (const UE::Json::FValue& SubnamespaceEntry : *SubnamespaceArray)
		{
			if (!SubnamespaceEntry.IsObject())
			{
				return false;
			}

			UE::Json::FConstObject SubnamespaceJSONObject = SubnamespaceEntry.GetObject();
			if (!JsonObjToArchive(SubnamespaceJSONObject, AccumulatedNamespace, InArchive, InManifest, InNativeArchive))
			{
				return false;
			}
		}
	}

	return true;
}


void FJsonInternationalizationArchiveSerializer::GenerateStructuredData( TSharedRef< const FInternationalizationArchive > InArchive, TSharedPtr<FStructuredArchiveEntry> RootElement )
{
	// Loop through all the unstructured archive entries and build up our structured hierarchy
	TArray<FString> NamespaceTokens;
	for (FArchiveEntryByStringContainer::TConstIterator It(InArchive->GetEntriesBySourceTextIterator()); It; ++It)
	{
		const TSharedRef<FArchiveEntry> UnstructuredArchiveEntry = It.Value();

		// Tokenize the namespace by using '.' as a delimiter
		NamespaceTokens.Reset();
		UnstructuredArchiveEntry->Namespace.GetString().ParseIntoArray(NamespaceTokens, *NAMESPACE_DELIMITER, true);

		TSharedPtr<FStructuredArchiveEntry> StructuredArchiveEntry = RootElement;

		// Loop through all the namespace tokens and find the appropriate structured entry, if it does not exist add it.
		// At the end StructuredArchiveEntry will point to the correct hierarchy entry for a given namespace
		for (const FString& NamespaceToken : NamespaceTokens)
		{
			TSharedPtr<FStructuredArchiveEntry> FoundNamespaceEntry = StructuredArchiveEntry->SubNamespaces.FindRef(NamespaceToken);
			if (!FoundNamespaceEntry)
			{
				FoundNamespaceEntry = StructuredArchiveEntry->SubNamespaces.Add(NamespaceToken, MakeShared<FStructuredArchiveEntry>(NamespaceToken));
			}
			StructuredArchiveEntry = MoveTemp(FoundNamespaceEntry);
		}

		// We add the unstructured Archive entry to the hierarchy
		StructuredArchiveEntry->ArchiveEntries.Add(UnstructuredArchiveEntry);
	}
}


void FJsonInternationalizationArchiveSerializer::SortStructuredData( TSharedPtr< FStructuredArchiveEntry > InElement )
{
	if( !InElement.IsValid() )
	{
		return;
	}

	// Sort the manifest entries by source text (primary) and key (secondary).
	InElement->ArchiveEntries.Sort(
		[](const TSharedPtr<FArchiveEntry>& A, const TSharedPtr<FArchiveEntry>& B)
		{
			if (A->Source == B->Source)
			{
				if (A->Key == B->Key)
				{
					if (A->KeyMetadataObj.IsValid() != B->KeyMetadataObj.IsValid())
					{
						return B->KeyMetadataObj.IsValid();
					}
					if (A->KeyMetadataObj.IsValid() && B->KeyMetadataObj.IsValid())
					{
						return (*A->KeyMetadataObj < *B->KeyMetadataObj);
					}
				}
				return A->Key < B->Key;
			}
			return A->Source < B->Source;
		});

	// Sort the subnamespaces by namespace string
	InElement->SubNamespaces.KeySort(
		[](const FString& A, const FString& B)
		{
			return A.Compare(B, ESearchCase::CaseSensitive) < 0;
		});

	// Do the sorting for each of the subnamespaces
	for( auto Iter = InElement->SubNamespaces.CreateIterator(); Iter; ++Iter )
	{
		TSharedPtr< FStructuredArchiveEntry > SubElement = Iter->Value;

		SortStructuredData( SubElement );
	}
}


void FJsonInternationalizationArchiveSerializer::StructuredDataToJsonObj( TSharedPtr< const FStructuredArchiveEntry > InElement, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator )
{
	JsonObj.AddMember(UE::Json::MakeStringRef(TAG_NAMESPACE), UE::Json::MakeStringValue(InElement->Namespace, Allocator), Allocator);

	// Write namespace content entries
	UE::Json::FValue EntryJSONArray(rapidjson::kArrayType);
	for (const TSharedPtr<FArchiveEntry>& Entry : InElement->ArchiveEntries)
	{
		UE::Json::FValue EntryJSONObject(rapidjson::kObjectType);

		{
			UE::Json::FValue SourceJSONObject(rapidjson::kObjectType);

			if (Entry->Source.MetadataObj)
			{
				FJsonInternationalizationMetaDataSerializer::SerializeMetadata(Entry->Source.MetadataObj.ToSharedRef(), SourceJSONObject, Allocator);
			}

			SourceJSONObject.AddMember(UE::Json::MakeStringRef(TAG_SOURCE_TEXT), UE::Json::MakeStringValue(Entry->Source.Text, Allocator), Allocator);

			EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_SOURCE), MoveTemp(SourceJSONObject), Allocator);
		}
		
		{
			UE::Json::FValue TranslationJSONObject(rapidjson::kObjectType);

			if (Entry->Translation.MetadataObj)
			{
				FJsonInternationalizationMetaDataSerializer::SerializeMetadata(Entry->Translation.MetadataObj.ToSharedRef(), TranslationJSONObject, Allocator);
			}

			TranslationJSONObject.AddMember(UE::Json::MakeStringRef(TAG_TRANSLATION_TEXT), UE::Json::MakeStringValue(Entry->Translation.Text, Allocator), Allocator);

			EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_TRANSLATION), MoveTemp(TranslationJSONObject), Allocator);
		}

		EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_KEY), UE::Json::MakeStringValue(Entry->Key.GetString(), Allocator), Allocator);

		if (Entry->KeyMetadataObj)
		{
			UE::Json::FValue MetaDataJSONObject(rapidjson::kObjectType);

			{
				UE::Json::FValue KeyDataJSONObject(rapidjson::kObjectType);
				FJsonInternationalizationMetaDataSerializer::SerializeMetadata(Entry->KeyMetadataObj.ToSharedRef(), KeyDataJSONObject, Allocator);
				if (KeyDataJSONObject.MemberCount() > 0)
				{
					MetaDataJSONObject.AddMember(UE::Json::MakeStringRef(TAG_METADATA_KEY), MoveTemp(KeyDataJSONObject), Allocator);
				}
			}

			if (MetaDataJSONObject.MemberCount() > 0)
			{
				EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_METADATA), MoveTemp(MetaDataJSONObject), Allocator);
			}
		}

		// We only add the optional field if it is true, it is assumed to be false otherwise.
		if (Entry->bIsOptional)
		{
			EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_OPTIONAL), UE::Json::FValue(Entry->bIsOptional), Allocator);
		}

		EntryJSONArray.PushBack(MoveTemp(EntryJSONObject), Allocator);
	}

	// Write the subnamespaces
	UE::Json::FValue NamespaceJSONArray(rapidjson::kArrayType);
	for (const TTuple<FString, TSharedPtr<FStructuredArchiveEntry>>& SubElementPair : InElement->SubNamespaces)
	{
		if (SubElementPair.Value)
		{
			UE::Json::FValue SubJSONObject(rapidjson::kObjectType);
			StructuredDataToJsonObj(SubElementPair.Value, SubJSONObject.GetObject(), Allocator);

			NamespaceJSONArray.PushBack(MoveTemp(SubJSONObject), Allocator);
		}
	}

	if (EntryJSONArray.Size() > 0)
	{
		JsonObj.AddMember(UE::Json::MakeStringRef(TAG_CHILDREN), MoveTemp(EntryJSONArray), Allocator);
	}

	if (NamespaceJSONArray.Size() > 0)
	{
		JsonObj.AddMember(UE::Json::MakeStringRef(TAG_SUBNAMESPACES), MoveTemp(NamespaceJSONArray), Allocator);
	}
}
