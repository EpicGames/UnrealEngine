// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonInternationalizationManifestSerializer.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonInternationalizationMetadataSerializer.h"


DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationManifestSerializer, Log, All);


const FString FJsonInternationalizationManifestSerializer::TAG_FORMATVERSION = TEXT("FormatVersion");
const FString FJsonInternationalizationManifestSerializer::TAG_NAMESPACE = TEXT("Namespace");
const FString FJsonInternationalizationManifestSerializer::TAG_CHILDREN = TEXT("Children");
const FString FJsonInternationalizationManifestSerializer::TAG_SUBNAMESPACES = TEXT("Subnamespaces");
const FString FJsonInternationalizationManifestSerializer::TAG_PATH = TEXT("Path");
const FString FJsonInternationalizationManifestSerializer::TAG_DEVNOTES = TEXT("DevNotes");
const FString FJsonInternationalizationManifestSerializer::TAG_OPTIONAL = TEXT("Optional");
const FString FJsonInternationalizationManifestSerializer::TAG_KEYCOLLECTION = TEXT("Keys");
const FString FJsonInternationalizationManifestSerializer::TAG_KEY = TEXT("Key");
const FString FJsonInternationalizationManifestSerializer::TAG_DEPRECATED_DEFAULTTEXT = TEXT("DefaultText");
const FString FJsonInternationalizationManifestSerializer::TAG_SOURCE = TEXT("Source");
const FString FJsonInternationalizationManifestSerializer::TAG_SOURCE_TEXT = TEXT("Text");
const FString FJsonInternationalizationManifestSerializer::TAG_METADATA = TEXT("MetaData");
const FString FJsonInternationalizationManifestSerializer::TAG_METADATA_INFO = TEXT("Info");
const FString FJsonInternationalizationManifestSerializer::TAG_METADATA_KEY = TEXT("Key");
const FString FJsonInternationalizationManifestSerializer::NAMESPACE_DELIMITER = TEXT(".");


bool FJsonInternationalizationManifestSerializer::DeserializeManifestFromString( const FString& InStr, TSharedRef< FInternationalizationManifest > Manifest, const FName PlatformName )
{
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Document = UE::Json::Parse(InStr);

	if (Document.HasError())
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to parse manifest. %ls.", *Document.GetError().CreateMessage(InStr));
		return false;
	}

	TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(Document.GetValue());
	if (!RootObject.IsSet())
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to parse manifest. No root object.");
		return false;
	}

	return DeserializeInternal(*RootObject, Manifest, PlatformName);
}


bool FJsonInternationalizationManifestSerializer::DeserializeManifestFromFile( const FString& InJsonFile, TSharedRef< FInternationalizationManifest > Manifest, const FName PlatformName )
{
	// Read in file as string
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *InJsonFile))
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to load manifest '%ls'.", *InJsonFile);
		return false;
	}

	// Grab the internal character array so we can do an insitu parse
	TArray<TCHAR> FileData = MoveTemp(FileContents.GetCharArray());
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Document = UE::Json::ParseInPlace(FileData);

	if (Document.HasError())
	{
		// Have to load the file again as the JSON contents was rewritten by the insitu parse
		ensure(FFileHelper::LoadFileToString(FileContents, *InJsonFile));
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to parse manifest '%ls'. %ls.", *InJsonFile, *Document.GetError().CreateMessage(FileContents));
		return false;
	}

	TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(Document.GetValue());
	if (!RootObject.IsSet())
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to parse manifest '%ls'. No root object.", *InJsonFile);
		return false;
	}

	return DeserializeInternal(*RootObject, Manifest, PlatformName);
}


bool FJsonInternationalizationManifestSerializer::SerializeManifestToString( TSharedRef< const FInternationalizationManifest > Manifest, FString& Str )
{
	UE::Json::FDocument Document(rapidjson::kObjectType);
	
	if (SerializeInternal(Manifest, Document.GetObject(), Document.GetAllocator()))
	{
		Str = UE::Json::WritePretty(Document);
		return true;
	}

	return false;
}


bool FJsonInternationalizationManifestSerializer::SerializeManifestToFile( TSharedRef< const FInternationalizationManifest > Manifest, const FString& InJsonFile )
{
	FString JsonString;
	if (!SerializeManifestToString(Manifest, JsonString))
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to serialize manifest '%ls'.", *InJsonFile);
		return false;
	}

	// Save the JSON string (force Unicode for our manifest and archive files)
	// TODO: Switch to UTF-8 by default, unless the file on-disk is already UTF-16
	if (!FFileHelper::SaveStringToFile(JsonString, *InJsonFile, FFileHelper::EEncodingOptions::ForceUnicode))
	{
		UE_LOGF(LogInternationalizationManifestSerializer, Error, "Failed to save manifest '%ls'.", *InJsonFile);
		return false;
	}

	return true;
}


void FJsonInternationalizationManifestSerializer::SortManifest(const TSharedRef<FInternationalizationManifest>& Manifest)
{
	TSharedRef<FStructuredEntry> RootElement = MakeShared<FStructuredEntry>(FString());

	// Convert the manifest into the structured data we use for JSON serialization
	GenerateStructuredData(Manifest, RootElement);
	SortStructuredData(RootElement);

	// Convert the structured data back into the in-memory manifest
	Manifest->ClearEntries();
	StructuredDataToManifest(RootElement, Manifest);
}


bool FJsonInternationalizationManifestSerializer::DeserializeInternal( UE::Json::FConstObject InJsonObj, TSharedRef< FInternationalizationManifest > Manifest, const FName PlatformName )
{
	if (TOptional<int32> FormatVersion = UE::Json::GetInt32Field(InJsonObj, *TAG_FORMATVERSION))
	{
		if (*FormatVersion > (int32)FInternationalizationManifest::EFormatVersion::Latest)
		{
			// Manifest is too new to be loaded!
			return false;
		}

		Manifest->SetFormatVersion(static_cast<FInternationalizationManifest::EFormatVersion>(*FormatVersion));
	}
	else
	{
		Manifest->SetFormatVersion(FInternationalizationManifest::EFormatVersion::Initial);
	}

	if (JsonObjToManifest(InJsonObj, FString(), Manifest, PlatformName))
	{
		// We've been upgraded to the latest format now...
		Manifest->SetFormatVersion(FInternationalizationManifest::EFormatVersion::Latest);
		return true;
	}

	return false;
}


bool FJsonInternationalizationManifestSerializer::SerializeInternal( TSharedRef< const FInternationalizationManifest > InManifest, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator )
{
	TSharedPtr< FStructuredEntry > RootElement = MakeShared<FStructuredEntry>(FString());

	// Condition the data so that it exists in a structured hierarchy for easy population of the JSON object.
	GenerateStructuredData( InManifest, RootElement );

	// Arrange the entries in non-cultural format so that diffs are easier to read.
	SortStructuredData( RootElement );

	// Clear out anything that may be in the JSON object
	JsonObj.RemoveAllMembers();

	// Set format version.
	JsonObj.AddMember(UE::Json::MakeStringRef(TAG_FORMATVERSION), UE::Json::FValue((int32)InManifest->GetFormatVersion()), Allocator);

	// Setup the JSON object using the structured data created
	StructuredDataToJsonObj( RootElement, JsonObj, Allocator );

	return true;
}


bool FJsonInternationalizationManifestSerializer::JsonObjToManifest( UE::Json::FConstObject InJsonObj, FString ParentNamespace, TSharedRef< FInternationalizationManifest > Manifest, const FName PlatformName )
{
	FString AccumulatedNamespace = ParentNamespace;

	if (TOptional<FStringView> Namespace = UE::Json::GetStringField(InJsonObj, *TAG_NAMESPACE))
	{
		if (!AccumulatedNamespace.IsEmpty())
		{
			AccumulatedNamespace += NAMESPACE_DELIMITER;
		}
		AccumulatedNamespace += *Namespace;
	}
	else
	{
		// We found an entry with a missing namespace
		return false;
	}

	// Process all the child objects
	if (TOptional<UE::Json::FConstArray> ChildrenArray = UE::Json::GetArrayField(InJsonObj, *TAG_CHILDREN))
	{
		for (const UE::Json::FValue& ChildEntry : *ChildrenArray)
		{
			if (!ChildEntry.IsObject())
			{
				return false;
			}

			UE::Json::FConstObject ChildJSONObject = ChildEntry.GetObject();

			FString SourceText;
			TSharedPtr< FLocMetadataObject > SourceMetadata;
			if (TOptional<UE::Json::FConstObject> SourceJSONObject = UE::Json::GetObjectField(ChildJSONObject, *TAG_SOURCE))
			{
				if (TOptional<FStringView> SourceTextValue = UE::Json::GetStringField(*SourceJSONObject, *TAG_SOURCE_TEXT))
				{
					SourceText = *SourceTextValue;

					// Source meta data is mixed in with the source text, we'll process metadata if the source json object has more than one entry
					if (SourceJSONObject->MemberCount() > 1)
					{
						// We load in the entire source object as metadata and just remove the source text.
						FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*SourceJSONObject, SourceMetadata);
						if (SourceMetadata)
						{
							SourceMetadata->Values.Remove(TAG_SOURCE_TEXT);
						}
					}
				}
				else
				{
					return false;
				}
			}
			else if (TOptional<FStringView> DefaultText = UE::Json::GetStringField(ChildJSONObject, *TAG_DEPRECATED_DEFAULTTEXT))
			{
				SourceText = *DefaultText;
			}
			else
			{
				return false;
			}

			FLocItem Source(SourceText);
			Source.MetadataObj = SourceMetadata;

			if (TOptional<UE::Json::FConstArray> ContextArray = UE::Json::GetArrayField(ChildJSONObject, *TAG_KEYCOLLECTION))
			{
				for (const UE::Json::FValue& ContextEntry : *ContextArray)
				{
					if (!ContextEntry.IsObject())
					{
						return false;
					}

					UE::Json::FConstObject ContextJSONObject = ContextEntry.GetObject();

					if (TOptional<FStringView> Key = UE::Json::GetStringField(ContextJSONObject, *TAG_KEY))
					{
						TOptional<FStringView> SourceLocation = UE::Json::GetStringField(ContextJSONObject, *TAG_PATH);

						FManifestContext CommandContext;
						CommandContext.Key = FString(*Key);
						CommandContext.SourceLocation = SourceLocation.Get(FString());
						CommandContext.PlatformName = PlatformName;
						CommandContext.DevNotes = UE::Json::GetStringField(ContextJSONObject, *TAG_DEVNOTES).Get(TEXTVIEW(""));

						if (TOptional<bool> IsOptionalValue = UE::Json::GetBoolField(ContextJSONObject, *TAG_OPTIONAL))
						{
							CommandContext.bIsOptional = *IsOptionalValue;
						}

						if (TOptional<UE::Json::FConstObject> MetaDataJSONObject = UE::Json::GetObjectField(ContextJSONObject, *TAG_METADATA))
						{
							if (TOptional<UE::Json::FConstObject> MetaDataInfoJSONObject = UE::Json::GetObjectField(*MetaDataJSONObject, *TAG_METADATA_INFO))
							{
								TSharedPtr< FLocMetadataObject > MetadataNode;
								FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*MetaDataInfoJSONObject, MetadataNode);
								if (MetadataNode)
								{
									CommandContext.InfoMetadataObj = MetadataNode;
								}
							}

							if (TOptional<UE::Json::FConstObject> MetaDataKeyJSONObject = UE::Json::GetObjectField(*MetaDataJSONObject, *TAG_METADATA_KEY))
							{
								TSharedPtr< FLocMetadataObject > MetadataNode;
								FJsonInternationalizationMetaDataSerializer::DeserializeMetadata(*MetaDataKeyJSONObject, MetadataNode);
								if (MetadataNode)
								{
									CommandContext.KeyMetadataObj = MetadataNode;
								}
							}
						}

						bool bAddSuccessful = Manifest->AddSource(AccumulatedNamespace, Source, CommandContext);
						if (!bAddSuccessful)
						{
							UE_LOGF(LogInternationalizationManifestSerializer, Warning, "Could not add JSON entry to the Internationalization manifest: Namespace:%ls SourceText:%ls SourceData:%ls",
								*AccumulatedNamespace,
								*Source.Text,
								*FJsonInternationalizationMetaDataSerializer::MetadataToString(Source.MetadataObj));
						}
					}
					else
					{
						//We found a context entry that is missing a identifier/key or a path
						return false;
					}

				}
			}
			else
			{
				// We have an entry that is missing a key/context collection or default text entry.
				return false;
			}

		}
	}

	if (TOptional<UE::Json::FConstArray> SubnamespaceArray = UE::Json::GetArrayField(InJsonObj, *TAG_SUBNAMESPACES))
	{
		for (const UE::Json::FValue& SubnamespaceEntry : *SubnamespaceArray)
		{
			if (!SubnamespaceEntry.IsObject())
			{
				return false;
			}

			UE::Json::FConstObject SubnamespaceJSONObject = SubnamespaceEntry.GetObject();
			if (!JsonObjToManifest(SubnamespaceJSONObject, AccumulatedNamespace, Manifest, PlatformName))
			{
				return false;
			}
		}
	}

	return true;
}


void FJsonInternationalizationManifestSerializer::GenerateStructuredData( TSharedRef< const FInternationalizationManifest > InManifest, TSharedPtr< FStructuredEntry > RootElement )
{
	// Loop through all the unstructured manifest entries and build up our structured hierarchy
	TArray<FString> NamespaceTokens;
	for (FManifestEntryByStringContainer::TConstIterator It(InManifest->GetEntriesBySourceTextIterator()); It; ++It)
	{
		const TSharedRef<FManifestEntry> UnstructuredManifestEntry = It.Value();

		// Tokenize the namespace by using '.' as a delimiter
		NamespaceTokens.Reset();
		UnstructuredManifestEntry->Namespace.GetString().ParseIntoArray(NamespaceTokens, *NAMESPACE_DELIMITER, true);

		TSharedPtr<FStructuredEntry> StructuredManifestEntry = RootElement;

		// Loop through all the namespace tokens and find the appropriate structured entry, if it does not exist add it.
		// At the end StructuredManifestEntry will point to the correct hierarchy entry for a given namespace
		for (const FString& NamespaceToken : NamespaceTokens)
		{
			TSharedPtr<FStructuredEntry> FoundNamespaceEntry = StructuredManifestEntry->SubNamespaces.FindRef(NamespaceToken);
			if (!FoundNamespaceEntry)
			{
				FoundNamespaceEntry = StructuredManifestEntry->SubNamespaces.Add(NamespaceToken, MakeShared<FStructuredEntry>(NamespaceToken));
			}
			StructuredManifestEntry = MoveTemp(FoundNamespaceEntry);
		}

		// We add the unstructured manifest entry to the hierarchy
		StructuredManifestEntry->ManifestEntries.Add(UnstructuredManifestEntry);
	}
}


void FJsonInternationalizationManifestSerializer::SortStructuredData( TSharedPtr< FStructuredEntry > InElement )
{
	if( !InElement.IsValid() )
	{
		return;
	}

	// Sort the manifest entries by source text.
	InElement->ManifestEntries.Sort(
		[](const TSharedPtr< FManifestEntry >& A, const TSharedPtr< FManifestEntry >& B)
		{
			return A->Source < B->Source;
		});

	// Sort the manifest entry contexts by key/identifier (primary) and source location (secondary)
	for( TArray< TSharedPtr< FManifestEntry > >::TIterator Iter( InElement->ManifestEntries.CreateIterator() ); Iter; ++Iter)
	{
		TSharedPtr< FManifestEntry > SubEntry = *Iter;
		if( SubEntry.IsValid())
		{
			SubEntry->Contexts.Sort(
				[](const FManifestContext& A, const FManifestContext& B)
				{
					if (A == B)
					{
						return A.SourceLocation.Compare(B.SourceLocation, ESearchCase::CaseSensitive) < 0;
					}
					return A < B;
				});
		}
	}
	
	// Sort the subnamespaces by namespace string
	InElement->SubNamespaces.KeySort(
		[](const FString& A, const FString& B)
		{
			return A.Compare(B, ESearchCase::CaseSensitive) < 0;
		});

	// Do the sorting for each of the subnamespaces
	for( auto Iter = InElement->SubNamespaces.CreateIterator(); Iter; ++Iter )
	{
		TSharedPtr< FStructuredEntry > SubElement = Iter->Value;

		SortStructuredData( SubElement );
	}
}


void FJsonInternationalizationManifestSerializer::StructuredDataToManifest(const TSharedPtr<const FStructuredEntry>& InElement, const TSharedRef<FInternationalizationManifest>& Manifest)
{
	if (!InElement)
	{
		return;
	}

	for (const TSharedPtr<FManifestEntry>& Entry : InElement->ManifestEntries)
	{
		for (const FManifestContext& Context : Entry->Contexts)
		{
			Manifest->AddSource(Entry->Namespace, Entry->Source, Context);
		}
	}

	for (const TTuple<FString, TSharedPtr<FStructuredEntry>>& SubElementPair : InElement->SubNamespaces)
	{
		StructuredDataToManifest(SubElementPair.Value, Manifest);
	}
}


void FJsonInternationalizationManifestSerializer::StructuredDataToJsonObj( TSharedPtr< const FStructuredEntry > InElement, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator )
{
	JsonObj.AddMember(UE::Json::MakeStringRef(TAG_NAMESPACE), UE::Json::MakeStringValue(InElement->Namespace, Allocator), Allocator);

	// Write namespace content entries
	UE::Json::FValue EntryJSONArray(rapidjson::kArrayType);
	for (const TSharedPtr<FManifestEntry>& Entry : InElement->ManifestEntries)
	{
		UE::Json::FValue EntryJSONObject(rapidjson::kObjectType);

		{
			UE::Json::FValue SourceJSONObject(rapidjson::kObjectType);

			if (Entry->Source.MetadataObj)
			{
				FJsonInternationalizationMetaDataSerializer::SerializeMetadata(Entry->Source.MetadataObj.ToSharedRef(), SourceJSONObject, Allocator);
			}

			SourceJSONObject.AddMember(UE::Json::MakeStringRef(TAG_SOURCE_TEXT), UE::Json::MakeStringValue(Entry->Source.Text, Allocator), Allocator);

			EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_SOURCE), MoveTemp(SourceJSONObject), Allocator);
		}

		UE::Json::FValue KeyJSONArray(rapidjson::kArrayType);

		for (const FManifestContext& AContext : Entry->Contexts)
		{
			FString ProcessedText = AContext.SourceLocation;
			ProcessedText.ReplaceInline( TEXT("\\"), TEXT("/"));
			ProcessedText.ReplaceInline( *FPaths::RootDir(), TEXT("/"));

			UE::Json::FValue KeyJSONObject(rapidjson::kObjectType);

			KeyJSONObject.AddMember(UE::Json::MakeStringRef(TAG_KEY), UE::Json::MakeStringValue(AContext.Key.GetString(), Allocator), Allocator);
			KeyJSONObject.AddMember(UE::Json::MakeStringRef(TAG_PATH), UE::Json::MakeStringValue(ProcessedText, Allocator), Allocator);
			if (!AContext.DevNotes.IsEmpty())
			{
				KeyJSONObject.AddMember(UE::Json::MakeStringRef(TAG_DEVNOTES), UE::Json::MakeStringValue(AContext.DevNotes, Allocator), Allocator);
			}

			// We only add the optional field if it is true, it is assumed to be false otherwise.
			if (AContext.bIsOptional)
			{
				KeyJSONObject.AddMember(UE::Json::MakeStringRef(TAG_OPTIONAL), UE::Json::FValue(AContext.bIsOptional), Allocator);
			}

			{
				UE::Json::FValue MetaDataJSONObject(rapidjson::kObjectType);

				if (AContext.InfoMetadataObj)
				{
					UE::Json::FValue InfoDataJSONObject(rapidjson::kObjectType);
					FJsonInternationalizationMetaDataSerializer::SerializeMetadata(AContext.InfoMetadataObj.ToSharedRef(), InfoDataJSONObject, Allocator);
					if (InfoDataJSONObject.MemberCount() > 0)
					{
						MetaDataJSONObject.AddMember(UE::Json::MakeStringRef(TAG_METADATA_INFO), MoveTemp(InfoDataJSONObject), Allocator);
					}
				}

				if (AContext.KeyMetadataObj)
				{
					UE::Json::FValue KeyDataJSONObject(rapidjson::kObjectType);
					FJsonInternationalizationMetaDataSerializer::SerializeMetadata(AContext.KeyMetadataObj.ToSharedRef(), KeyDataJSONObject, Allocator);
					if (KeyDataJSONObject.MemberCount() > 0)
					{
						MetaDataJSONObject.AddMember(UE::Json::MakeStringRef(TAG_METADATA_KEY), MoveTemp(KeyDataJSONObject), Allocator);
					}
				}

				if (MetaDataJSONObject.MemberCount() > 0)
				{
					KeyJSONObject.AddMember(UE::Json::MakeStringRef(TAG_METADATA), MoveTemp(MetaDataJSONObject), Allocator);
				}
			}
			
			KeyJSONArray.PushBack(MoveTemp(KeyJSONObject), Allocator);
		}

		EntryJSONObject.AddMember(UE::Json::MakeStringRef(TAG_KEYCOLLECTION), MoveTemp(KeyJSONArray), Allocator);

		EntryJSONArray.PushBack(MoveTemp(EntryJSONObject), Allocator);
	}

	// Write the subnamespaces
	UE::Json::FValue NamespaceJSONArray(rapidjson::kArrayType);
	for (const TTuple<FString, TSharedPtr<FStructuredEntry>>& SubElementPair : InElement->SubNamespaces)
	{
		if (SubElementPair.Value)
		{
			UE::Json::FValue SubJSONObject(rapidjson::kObjectType);
			StructuredDataToJsonObj(SubElementPair.Value, SubJSONObject.GetObject(), Allocator);

			NamespaceJSONArray.PushBack(MoveTemp(SubJSONObject), Allocator);
		}
	}

	if (EntryJSONArray.Size() > 0)
	{
		JsonObj.AddMember(UE::Json::MakeStringRef(TAG_CHILDREN), MoveTemp(EntryJSONArray), Allocator);
	}

	if (NamespaceJSONArray.Size() > 0)
	{
		JsonObj.AddMember(UE::Json::MakeStringRef(TAG_SUBNAMESPACES), MoveTemp(NamespaceJSONArray), Allocator);
	}
}
