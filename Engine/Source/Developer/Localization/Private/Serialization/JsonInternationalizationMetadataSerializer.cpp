// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonInternationalizationMetadataSerializer.h"


struct FCompareLocMetadataValue
{
	FORCEINLINE bool operator()( TSharedPtr< FLocMetadataValue > A, TSharedPtr< FLocMetadataValue > B ) const
	{
		if( A.IsValid() && B.IsValid() )
		{
			return (*A < *B);
		}
		else if( A.IsValid() != B.IsValid() )
		{
			return B.IsValid();
		}
		return false;
	}
};


TSharedPtr<FLocMetadataValue> JsonValueToLocMetaDataValue(const UE::Json::FValue& JsonValue)
{
	switch (JsonValue.GetType())
	{
		case rapidjson::kFalseType:
		case rapidjson::kTrueType:
		{
			return MakeShared<FLocMetadataValueBoolean>(JsonValue.GetBool());
		}

		case rapidjson::kStringType:
		{
			return MakeShared<FLocMetadataValueString>(FString(UE::Json::ValueAsStringView(JsonValue)));
		}

		case rapidjson::kArrayType:
		{
			TArray<TSharedPtr<FLocMetadataValue>> MetadataArray;
			UE::Json::FConstArray JsonArray = JsonValue.GetArray();
			for (const UE::Json::FValue& Item : JsonArray)
			{
				if (TSharedPtr<FLocMetadataValue> Entry = JsonValueToLocMetaDataValue(Item))
				{
					MetadataArray.Add(Entry);
				}
			}
			if (MetadataArray.Num() > 0)
			{
				return MakeShared<FLocMetadataValueArray>(MoveTemp(MetadataArray));
			}
		}
		break;

		case rapidjson::kObjectType:
		{
			TSharedPtr<FLocMetadataObject> MetadataSubObject = MakeShared<FLocMetadataObject>();
			UE::Json::FConstObject JsonObject = JsonValue.GetObject();
			for (const UE::Json::FMember& ValueIter : JsonObject)
			{
				if (TSharedPtr<FLocMetadataValue> MetadataValue = JsonValueToLocMetaDataValue(ValueIter.value))
				{
					MetadataSubObject->SetField(FString(UE::Json::ValueAsStringView(ValueIter.name)), MetadataValue);
				}
			}
			return MakeShared<FLocMetadataValueObject>(MoveTemp(MetadataSubObject));
		}

		default:
		{
			// At the moment we do not support all the json types as metadata.  In the future these types will be handled in a way that they can be stored in an unprocessed way.
		}
		break;
	}

	return nullptr;
}


TOptional<UE::Json::FValue> LocMetaDataValueToJsonValue(const TSharedRef< FLocMetadataValue > MetadataValue, UE::Json::FValue* OutTopLevelObjectValuePtr, UE::Json::FAllocator& Allocator)
{
	switch (MetadataValue->GetType())
	{
		case ELocMetadataType::Boolean:
		{
			return UE::Json::FValue(MetadataValue->AsBool());
		}

		case ELocMetadataType::String:
		{
			return UE::Json::MakeStringValue(MetadataValue->AsString(), Allocator);
		}

		case ELocMetadataType::Array:
		{
			TArray< TSharedPtr< FLocMetadataValue > > MetaDataArray = MetadataValue->AsArray();
			MetaDataArray.Sort(FCompareLocMetadataValue());

			UE::Json::FValue JsonArrayVals(rapidjson::kArrayType);
			for (const TSharedPtr<FLocMetadataValue>& Item : MetaDataArray)
			{
				if (Item)
				{
					if (TOptional<UE::Json::FValue> Entry = LocMetaDataValueToJsonValue(Item.ToSharedRef(), nullptr, Allocator))
					{
						JsonArrayVals.PushBack(MoveTemp(*Entry), Allocator);
					}
				}
			}

			if (JsonArrayVals.Size() > 0)
			{
				return JsonArrayVals;
			}
		}
		break;

		case ELocMetadataType::Object:
		{
			TMap<FString, TSharedPtr<FLocMetadataValue>> MetaDataMap = MetadataValue->AsObject()->Values;
			// Sorting by key is probably sufficient for now but ideally we would sort the resulting json object using
			//  the same logic that is in the FLocMetadata < operator
			MetaDataMap.KeySort(TLess<FString>());

			TOptional<UE::Json::FValue> JsonSubObject;
			if (!OutTopLevelObjectValuePtr)
			{
				JsonSubObject.Emplace(rapidjson::kObjectType);
			}

			UE::Json::FValue& JsonObject = OutTopLevelObjectValuePtr ? *OutTopLevelObjectValuePtr : JsonSubObject.GetValue();
			check(JsonObject.IsObject());
			for (const TTuple<FString, TSharedPtr<FLocMetadataValue>>& MetaDataPair : MetaDataMap)
			{
				if (MetaDataPair.Value)
				{
					if (TOptional<UE::Json::FValue> JsonValue = LocMetaDataValueToJsonValue(MetaDataPair.Value.ToSharedRef(), nullptr, Allocator))
					{
						JsonObject.AddMember(UE::Json::MakeStringValue(MetaDataPair.Key, Allocator), MoveTemp(*JsonValue), Allocator);
					}
				}
			}

			return JsonSubObject;
		}

		default:
			break;
	}

	return {};
}


void FJsonInternationalizationMetaDataSerializer::DeserializeMetadata( UE::Json::FConstObject JsonObj, TSharedPtr< FLocMetadataObject >& OutMetaDataObj )
{
	OutMetaDataObj = JsonValueToLocMetaDataValue(JsonObj)->AsObject();
}


void FJsonInternationalizationMetaDataSerializer::SerializeMetadata( const TSharedRef< FLocMetadataObject > MetaData, UE::Json::FValue& OutJsonObj, UE::Json::FAllocator& Allocator )
{
	check(OutJsonObj.IsObject());
	LocMetaDataValueToJsonValue(MakeShared<FLocMetadataValueObject>(MetaData), &OutJsonObj, Allocator);
}


FString FJsonInternationalizationMetaDataSerializer::MetadataToString( const TSharedPtr<FLocMetadataObject> Metadata )
{
	FString StringMetadata;
	if (Metadata)
	{
		UE::Json::FDocument Document(rapidjson::kObjectType);

		FJsonInternationalizationMetaDataSerializer::SerializeMetadata(Metadata.ToSharedRef(), Document, Document.GetAllocator());
		if (Document.MemberCount() > 0)
		{
			StringMetadata = UE::Json::WritePretty(Document);
			StringMetadata.ReplaceInline(TEXT("\t"), TEXT(" "));
			StringMetadata.ReplaceInline(TEXT("\r\n"), TEXT(" "));
			StringMetadata.ReplaceInline(TEXT("\n"), TEXT(" "));
		}
	}
	return StringMetadata;
}
