// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextLocalizationResourceGenerator.h"

#include "Internationalization/Culture.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextLocalizationResource.h"
#include "LocTextHelper.h"
#include "Logging/StructuredLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextLocalizationResourceGenerator, Log, All);
namespace TextLocalizationResourceGenerator
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

namespace UE::TextLocalizationResourceGenerator::Private
{
	// This function counts the number of opening rich text tags "<TagName>" and closing rich text tags "</>" in the rich text string InText
	void CountRichTextTags(const FStringView& InText, int& OutOpeningRichTextTagCount, int& OutClosingRighTextTagCount)
	{
		TCHAR PreviousChar = 0;
		TCHAR CurrentChar = 0;
		int TagLength = 0;
		bool bTagOpen = false;
		for (int CharacterIndex = 0; CharacterIndex < InText.Len(); ++CharacterIndex)
		{
			CurrentChar = InText[CharacterIndex];
			if (CurrentChar == TEXT('<'))
			{
				// Even if a tag was already "Opened" with a '<', then we assume it was not really a tag but now it might be.
				bTagOpen = true;
				TagLength = 0;
			}
			else if (bTagOpen)
			{
				if (CurrentChar == TEXT('>'))
				{
					if (InText[CharacterIndex - 1] == TEXT('/'))
					{
						if (TagLength == 1)
						{
							OutClosingRighTextTagCount++;
						}
						// else it is a self-closing tag and we don't count it
					}
					else
					{
						// There is one other exception: we need to support "<br>" as a self-closing tag.
						// It was supported in Text Render Components before multi-edit editable text and we need to support it for historic reasons.
						bool bIsBRTag = (TagLength == 2 && InText[CharacterIndex - 2] == 'b' && InText[CharacterIndex - 1] == 'r');

						if (!bIsBRTag)
						{
							OutOpeningRichTextTagCount++;
						}
					}

					bTagOpen = false;
				}
				TagLength++;
			}
		}
	}

	// Check that rich text tags are complete, balanced and symmetric across the source and one of its translation. (i.e. <text color="FFFFFFFF"> TEST </>)
	bool ValidateRichTextTags(const FStringView& InSource, const FStringView& InTranslation)
	{
		int TranslationRichTextOpeningTagCount = 0;
		int TranslationRichTextClosingTagCount = 0;
		CountRichTextTags(InTranslation, TranslationRichTextOpeningTagCount, TranslationRichTextClosingTagCount);
		if (TranslationRichTextOpeningTagCount != TranslationRichTextClosingTagCount) // The opening and closing tags are not balanced. Could be invalid.
		{
			// Check if the source has the same "issue". If so, we  might tolerate it as it probably is a special edgecase that 
			// is desirable and we can't cover all edgecases (i.e. maybe it is unbalanced for concatenation).
			int SourceRichTextOpenTagCount = 0;
			int SourceRichTextClosedTagCount = 0;
			CountRichTextTags(InSource, SourceRichTextOpenTagCount, SourceRichTextClosedTagCount);
			// We consider that it is not a translation mistake but it is voluntary if the source follow the same pattern.
			bool SourceHasTheSameCount = SourceRichTextOpenTagCount == TranslationRichTextOpeningTagCount && SourceRichTextClosedTagCount == TranslationRichTextClosingTagCount;
			if (!SourceHasTheSameCount)
			{
				// Now, we can safely assume it is really a translation error.
				return false;
			}
		}

		return true;
	}
}

bool FTextLocalizationResourceGenerator::GenerateLocMeta(const FLocTextHelper& InLocTextHelper, const FString& InResourceName, FTextLocalizationMetaDataResource& OutLocMeta)
{
	// Populate the meta-data
	OutLocMeta.NativeCulture = InLocTextHelper.GetNativeCulture();
	OutLocMeta.NativeLocRes = OutLocMeta.NativeCulture / InResourceName;
	OutLocMeta.CompiledCultures = InLocTextHelper.GetAllCultures();
	OutLocMeta.CompiledCultures.Sort();

	return true;
}

bool FTextLocalizationResourceGenerator::GenerateLocRes(const FLocTextHelper& InLocTextHelper, const FString& InCultureToGenerate, const EGenerateLocResFlags InGenerateFlags, const FTextKey& InLocResID, FTextLocalizationResource& OutPlatformAgnosticLocRes, TMap<FName, TSharedRef<FTextLocalizationResource>>& OutPerPlatformLocRes, const int32 InPriority)
{
	const bool bIsNativeCulture = InCultureToGenerate == InLocTextHelper.GetNativeCulture();
	FCulturePtr Culture = FInternationalization::Get().GetCulture(InCultureToGenerate);

	if (!Culture)
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Warning, "'{cultureCode}' did not resolve to a known culture",
			("cultureCode", InCultureToGenerate),
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	TArray<FString> InheritedCultures = Culture->GetPrioritizedParentCultureNames();
	InheritedCultures.Remove(Culture->GetName());

	// Always add the split platforms so that they generate an empty LocRes if there are no entries for that platform in the platform agnostic manifest
	for (const FString& SplitPlatformName : InLocTextHelper.GetPlatformsToSplit())
	{
		const FName SplitPlatformFName = *SplitPlatformName;
		if (!OutPerPlatformLocRes.Contains(SplitPlatformFName))
		{
			OutPerPlatformLocRes.Add(SplitPlatformFName, MakeShared<FTextLocalizationResource>());
		}
	}
	
	// calculate this ahead of time - we only need it once per InLocResID - this saves lots of wasted string manipulation and locking in FTextLocalizationResource::AddEntry
	const int32 LocalizationTargetPathID = TextLocalizationResourceUtil::GetLocalizationTargetPathIdFromLocResId(InLocResID);
	
	// Add each manifest entry to the LocRes file
	InLocTextHelper.EnumerateSourceTexts([&InLocTextHelper, &InCultureToGenerate, InGenerateFlags, &InLocResID, LocalizationTargetPathID, &OutPlatformAgnosticLocRes, &OutPerPlatformLocRes, InPriority, bIsNativeCulture, Culture, &InheritedCultures](TSharedRef<FManifestEntry> InManifestEntry) -> bool
	{
		// For each context, we may need to create a different or even multiple LocRes entries.
		for (const FManifestContext& Context : InManifestEntry->Contexts)
		{
			// Find the correct translation based upon the native source text
			FLocItem TranslationText;
			InLocTextHelper.GetRuntimeText(InCultureToGenerate, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, ELocTextExportSourceMethod::NativeText, InManifestEntry->Source, TranslationText, EnumHasAnyFlags(InGenerateFlags, EGenerateLocResFlags::AllowStaleTranslations));

			// Is this entry considered translated? Native entries are always translated
			bool bIsTranslated = bIsNativeCulture || !InManifestEntry->Source.IsExactMatch(TranslationText);
			if (!bIsTranslated && InheritedCultures.Num() > 0)
			{
				// If this entry has parent languages then we also need to test whether the current translation is different from any parent that we have translations for, 
				// as it may be that the translation was explicitly changed back to being the native text for some reason (eg, es-419 needs something in English that es translates)
				for (const FString& InheritedCulture : InheritedCultures)
				{
					if (InLocTextHelper.HasArchive(InheritedCulture))
					{
						FLocItem InheritedText;
						InLocTextHelper.GetRuntimeText(InheritedCulture, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, ELocTextExportSourceMethod::NativeText, InManifestEntry->Source, InheritedText, EnumHasAnyFlags(InGenerateFlags, EGenerateLocResFlags::AllowStaleTranslations));
						if (!InheritedText.IsExactMatch(TranslationText))
						{
							bIsTranslated = true;
							break;
						}
					}
				}
			}

			if (bIsTranslated)
			{
				// Validate translations that look like they could be format patterns
				if (EnumHasAnyFlags(InGenerateFlags, EGenerateLocResFlags::ValidateFormatPatterns) && TranslationText.Text.Contains(TEXT("{"), ESearchCase::CaseSensitive))
				{
					const FTextFormat FmtPattern = FTextFormat::FromString(TranslationText.Text);

					TArray<FString> ValidationErrors;
					if (!FmtPattern.ValidatePattern(Culture, ValidationErrors))
					{
						FString ValidationErrorsText = TEXT("");
						for (const FString& ValidationError : ValidationErrors)
						{
							ValidationErrorsText += FString::Printf(TEXT("\n  - %s"), *ValidationError);
						}
						UE_LOGFMT(LogTextLocalizationResourceGenerator, Warning, "{location}: Format pattern '{text}' ({locNamespace},{locKey}) generated the following validation errors for '{cultureCode}': {error}",
							("location", *InLocTextHelper.GetTargetPath() + FString("/") + InCultureToGenerate + FString("/") + *InLocTextHelper.GetTargetName() + FString(".archive")),
							("cultureCode", *InCultureToGenerate),
							("locNamespace", *InManifestEntry->Namespace.GetString()),
							("locKey", *Context.Key.GetString()),
							("text", *FLocTextHelper::SanitizeLogOutput(TranslationText.Text)),
							("error", *FLocTextHelper::SanitizeLogOutput(ValidationErrorsText)),
							("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
						);
					}
				}

				// Validate that text doesn't have leading or trailing whitespace
				if (EnumHasAnyFlags(InGenerateFlags, EGenerateLocResFlags::ValidateSafeWhitespace) && TranslationText.Text.Len() > 0)
				{
					auto IsUnsafeWhitespace = [](const TCHAR InChar)
						{
							// Unsafe whitespace is any whitespace character, except new-lines
							return FTextChar::IsWhitespace(InChar) && !(InChar == TEXT('\r') || InChar == TEXT('\n'));
						};

					if (IsUnsafeWhitespace(TranslationText.Text[0]) || IsUnsafeWhitespace(TranslationText.Text[TranslationText.Text.Len() - 1]))
					{
						UE_LOGFMT(LogTextLocalizationResourceGenerator, Warning, "{location}: Translation '{text}' ({locNamespace},{locKey}) has leading or trailing whitespace for '{cultureCode}'.",
							("location", *InLocTextHelper.GetTargetPath() + FString("/") + InCultureToGenerate + FString("/") + *InLocTextHelper.GetTargetName() + FString(".archive")),
							("cultureCode", *InCultureToGenerate),
							("locNamespace", *InManifestEntry->Namespace.GetString()),
							("locKey", *Context.Key.GetString()),
							("text", *FLocTextHelper::SanitizeLogOutput(TranslationText.Text)),
							("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
						);
					}
				}

				if (!bIsNativeCulture && EnumHasAnyFlags(InGenerateFlags, EGenerateLocResFlags::ValidateRichTextTags) &&
					!UE::TextLocalizationResourceGenerator::Private::ValidateRichTextTags(InManifestEntry->Source.Text, TranslationText.Text))
				{
					UE_LOGFMT(LogTextLocalizationResourceGenerator, Warning, "Broken Rich Text Tag detected in a translation. An unbalanced tag (a complete/incomplet opening rich text tag (i.e. <TagName>) with an incomplet/complete closing tag(</>)) was detected in a translation but not in its source text. Find the problematic tag in the translation and fix the translation to remove this warning. Translation File:'{translationFile}' Namespace And Key:'{locNamespace},{locKey}' Translation Text To Fix:'{text}'.",
						("cultureCode", *InCultureToGenerate),
						("locNamespace", *InManifestEntry->Namespace.GetString()),
						("locKey", *Context.Key.GetString()),
						("translationFile", *InLocTextHelper.GetTargetPath() + FString("/") + InCultureToGenerate + FString("/") + *InLocTextHelper.GetTargetName() + FString(".archive")),
						("text", *FLocTextHelper::SanitizeLogOutput(TranslationText.Text)),
						("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
					);
				}
			}

			// Find the LocRes to update
			FTextLocalizationResource* LocResToUpdate = &OutPlatformAgnosticLocRes;
			if (!Context.PlatformName.IsNone())
			{
				if (TSharedRef<FTextLocalizationResource>* PerPlatformLocRes = OutPerPlatformLocRes.Find(Context.PlatformName))
				{
					LocResToUpdate = &PerPlatformLocRes->Get();
				}
			}
			check(LocResToUpdate);

			// Add this entry to the LocRes
			LocResToUpdate->AddEntry(InManifestEntry->Namespace.GetString(), Context.Key.GetString(), InManifestEntry->Source.Text, TranslationText.Text, InPriority, InLocResID, LocalizationTargetPathID);
		}

		return true; // continue enumeration
	}, true);

	return true;
}

bool FTextLocalizationResourceGenerator::GenerateLocResAndUpdateLiveEntriesFromConfig(const FString& InConfigFilePath, const EGenerateLocResFlags InGenerateFlags)
{
	FInternationalization& I18N = FInternationalization::Get();

	const FString SectionName = TEXT("RegenerateResources");

	// Get native culture.
	FString NativeCulture;
	if (!GConfig->GetString(*SectionName, TEXT("NativeCulture"), NativeCulture, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No native culture specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get source path.
	FString SourcePath;
	if (!GConfig->GetString(*SectionName, TEXT("SourcePath"), SourcePath, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No source path specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get destination path.
	FString DestinationPath;
	if (!GConfig->GetString(*SectionName, TEXT("DestinationPath"), DestinationPath, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No destination path specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get manifest name.
	FString ManifestName;
	if (!GConfig->GetString(*SectionName, TEXT("ManifestName"), ManifestName, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No manifest name specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get archive name.
	FString ArchiveName;
	if (!GConfig->GetString(*SectionName, TEXT("ArchiveName"), ArchiveName, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No archive name specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get resource name.
	FString ResourceName;
	if (!GConfig->GetString(*SectionName, TEXT("ResourceName"), ResourceName, InConfigFilePath))
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No resource name specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Source path needs to be relative to Engine or Game directory
	const FString ConfigFullPath = FPaths::ConvertRelativePathToFull(InConfigFilePath);
	const FString EngineFullPath = FPaths::ConvertRelativePathToFull(FPaths::EngineConfigDir());
	const bool IsEngineManifest = ConfigFullPath.StartsWith(EngineFullPath);

	if (IsEngineManifest)
	{
		SourcePath = FPaths::Combine(*FPaths::EngineDir(), *SourcePath);
		DestinationPath = FPaths::Combine(*FPaths::EngineDir(), *DestinationPath);
	}
	else
	{
		SourcePath = FPaths::Combine(*FPaths::ProjectDir(), *SourcePath);
		DestinationPath = FPaths::Combine(*FPaths::ProjectDir(), *DestinationPath);
	}

	TArray<FString> CulturesToGenerate;
	{
		const FString CultureName = I18N.GetCurrentCulture()->GetName();
		const TArray<FString> PrioritizedCultures = I18N.GetPrioritizedCultureNames(CultureName);
		for (const FString& PrioritizedCulture : PrioritizedCultures)
		{
			if (FPaths::FileExists(SourcePath / PrioritizedCulture / ArchiveName))
			{
				CulturesToGenerate.Add(PrioritizedCulture);
			}
		}
	}

	if (CulturesToGenerate.Num() == 0)
	{
		UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "No cultures to generate were specified.",
			("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
		);
		return false;
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, NativeCulture, CulturesToGenerate, nullptr);
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGFMT(LogTextLocalizationResourceGenerator, Warning, "Load error: {error}",
				("error", *LoadError.ToString()),
				("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
			);
			return false;
		}
	}

	FTextLocalizationResource TextLocalizationResource;
	TMap<FName, TSharedRef<FTextLocalizationResource>> Unused_PerPlatformLocRes;
	for (int32 CultureIndex = 0; CultureIndex < CulturesToGenerate.Num(); ++CultureIndex)
	{
		const FString& CultureName = CulturesToGenerate[CultureIndex];

		const FString CulturePath = DestinationPath / CultureName;
		const FString ResourceFilePath = FPaths::ConvertRelativePathToFull(CulturePath / ResourceName);

		if (!GenerateLocRes(LocTextHelper, CultureName, InGenerateFlags, FTextKey(ResourceFilePath), TextLocalizationResource, Unused_PerPlatformLocRes, CultureIndex))
		{
			UE_LOGFMT(LogTextLocalizationResourceGenerator, Error, "Failed to generate localization resource for culture '{cultureName}'.",
				("cultureName", *CultureName),
				("id", TextLocalizationResourceGenerator::LocalizationLogIdentifier)
			);
			return false;
		}
	}
	FTextLocalizationManager::Get().UpdateFromLocalizationResource(TextLocalizationResource);

	return true;
}
