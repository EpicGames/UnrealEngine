// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserGeneratedContentLocalization.h"

#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/xxhash.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "JsonObjectConverter.h"
#include "Interfaces/IPluginManager.h"

#include "Internationalization/Culture.h"
#include "Internationalization/CultureFilter.h"
#include "Internationalization/TextLocalizationManager.h"
#include "TextLocalizationResourceGenerator.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationDelegates.h"
#include "LocalizationFileUtil.h"
#include "LocalizationSourceControlUtil.h"
#include "LocTextHelper.h"
#include "PortableObjectFormatDOM.h"
#include "SourceControlHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserGeneratedContentLocalization)

#define LOCTEXT_NAMESPACE "UserGeneratedContentLocalization"

DEFINE_LOG_CATEGORY_STATIC(LogUGCLocalization, Log, All);

namespace UserGeneratedContentLocalization
{

bool bAlwaysExportFullGatherLog = false;
FAutoConsoleVariableRef CExportFullGatherLog(TEXT("Localization.UGC.AlwaysExportFullGatherLog"), bAlwaysExportFullGatherLog, TEXT("True to export the full gather log from running localization commandlet, even if there we no errors"));

}

void FUserGeneratedContentLocalizationDescriptor::InitializeFromProject(const ELocalizedTextSourceCategory LocalizationCategory)
{
	ELocalizationLoadFlags LoadFlags = ELocalizationLoadFlags::None;
	switch (LocalizationCategory)
	{
	case ELocalizedTextSourceCategory::Game:
		LoadFlags |= ELocalizationLoadFlags::Game;
		break;

	case ELocalizedTextSourceCategory::Engine:
		LoadFlags |= ELocalizationLoadFlags::Engine;
		break;

	case ELocalizedTextSourceCategory::Editor:
		LoadFlags |= ELocalizationLoadFlags::Editor;
		break;

	default:
		checkf(false, TEXT("Unexpected ELocalizedTextSourceCategory!"));
		break;
	}

	NativeCulture = FTextLocalizationManager::Get().GetNativeCultureName(LocalizationCategory);
	if (NativeCulture.IsEmpty())
	{
		NativeCulture = TEXT("en");
	}
	CulturesToGenerate = FTextLocalizationManager::Get().GetLocalizedCultureNames(LoadFlags);

	// Filter any cultures that are disabled in shipping or via UGC loc settings
	{
		const FCultureFilter CultureFilter(EBuildConfiguration::Shipping, ELocalizationLoadFlags::Engine | LoadFlags);
		CulturesToGenerate.RemoveAll([&CultureFilter](const FString& Culture)
		{
			return !CultureFilter.IsCultureAllowed(Culture)
				|| GetDefault<UUserGeneratedContentLocalizationSettings>()->CulturesToDisable.Contains(Culture);
		});
	}
}

bool FUserGeneratedContentLocalizationDescriptor::Validate(const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor)
{
	int32 NumCulturesFixed = 0;

	if (!DefaultDescriptor.CulturesToGenerate.Contains(NativeCulture))
	{
		++NumCulturesFixed;
		NativeCulture = DefaultDescriptor.NativeCulture;
	}

	NumCulturesFixed += CulturesToGenerate.RemoveAll([&DefaultDescriptor](const FString& Culture)
	{
		return !DefaultDescriptor.CulturesToGenerate.Contains(Culture);
	});

	return NumCulturesFixed == 0;
}

bool FUserGeneratedContentLocalizationDescriptor::ToJsonObject(TSharedPtr<FJsonObject>& OutJsonObject) const
{
	OutJsonObject = FJsonObjectConverter::UStructToJsonObject(*this);
	return OutJsonObject.IsValid();
}

bool FUserGeneratedContentLocalizationDescriptor::ToJsonString(FString& OutJsonString) const
{
	return FJsonObjectConverter::UStructToJsonObjectString(*this, OutJsonString);
}

bool FUserGeneratedContentLocalizationDescriptor::ToJsonFile(const TCHAR* InFilename) const
{
	FString UGCLocDescData;
	return ToJsonString(UGCLocDescData) 
		&& FFileHelper::SaveStringToFile(UGCLocDescData, InFilename, FFileHelper::EEncodingOptions::ForceUTF8);
}

bool FUserGeneratedContentLocalizationDescriptor::FromJsonObject(TSharedRef<const FJsonObject> InJsonObject)
{
	return FJsonObjectConverter::JsonObjectToUStruct(ConstCastSharedRef<FJsonObject>(InJsonObject), this);
}

bool FUserGeneratedContentLocalizationDescriptor::FromJsonString(const FString& InJsonString)
{
	return FJsonObjectConverter::JsonObjectStringToUStruct(InJsonString, this);
}

bool FUserGeneratedContentLocalizationDescriptor::FromJsonFile(const TCHAR* InFilename)
{
	FString UGCLocDescData;
	return FFileHelper::LoadFileToString(UGCLocDescData, InFilename)
		&& FromJsonString(UGCLocDescData);
}

namespace UserGeneratedContentLocalization
{

FString GetLocalizationScratchDirectory()
{
	return FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("Localization"));
}

FString GetLocalizationScratchDirectory(const FString& LocalizationTargetName)
{
	return FPaths::Combine(GetLocalizationScratchDirectory(), LocalizationTargetName);
}

FStringView GetLocalizationTargetNameFieldName()
{
	return TEXTVIEW("UGCLocalizationTargetName");
}

FString GetLocalizationTargetName(const TSharedRef<IPlugin>& Plugin)
{
	FString LocalizationTargetName;

#if WITH_EDITOR
	if (const TSharedPtr<FJsonObject>& CachedDescriptorJson = Plugin->GetDescriptorJson())
	{
		CachedDescriptorJson->TryGetStringField(GetLocalizationTargetNameFieldName(), LocalizationTargetName);
	}
	else
#endif // WITH_EDITOR
	{
		FString FileContents;
		if (FFileHelper::LoadFileToString(FileContents, *Plugin->GetDescriptorFileName()))
		{
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
			TSharedPtr<FJsonObject> DescriptorJson;
			if (FJsonSerializer::Deserialize(Reader, DescriptorJson) && DescriptorJson.IsValid())
			{
				DescriptorJson->TryGetStringField(GetLocalizationTargetNameFieldName(), LocalizationTargetName);
			}
		}
	}

	if (!LocalizationTargetName.IsEmpty())
	{
		return LocalizationTargetName;
	}

	// Note: If you change this naming scheme you'll need to handle backwards computability with existing data
	return Plugin->GetName();
}

#if WITH_EDITOR

bool SetLocalizationTargetName(const TSharedRef<IPlugin>& Plugin, const FString& LocalizationTargetName, bool bUseSourceControl, FText& OutFailReason)
{
	FPluginDescriptor PluginDescriptor = Plugin->GetDescriptor();
	if (!LocalizationTargetName.IsEmpty())
	{
		PluginDescriptor.AdditionalFieldsToWrite.Emplace(GetLocalizationTargetNameFieldName(), MakeShared<FJsonValueString>(LocalizationTargetName));
	}
	else
	{
		PluginDescriptor.AdditionalFieldsToRemove.Emplace(GetLocalizationTargetNameFieldName());
	}

	FLocFileSCCNotifies LocFileNotifies = CreateLocFileNotifies(bUseSourceControl);

	LocFileNotifies.PreFileWrite(Plugin->GetDescriptorFileName());
	const bool bResult = Plugin->UpdateDescriptor(PluginDescriptor, OutFailReason);
	LocFileNotifies.PostFileWrite(Plugin->GetDescriptorFileName());
	return bResult;
}

#endif // WITH_EDITOR

FString GetLocalizationTargetDirectory(const TSharedRef<IPlugin>& Plugin)
{
	return GetLocalizationTargetDirectory(GetLocalizationTargetName(Plugin), Plugin->GetContentDir());
}

FString GetLocalizationTargetDirectory(const TSharedRef<IPlugin>& Plugin, const FString& LocalizationTargetRootDirectoryOverride)
{
	return GetLocalizationTargetDirectory(GetLocalizationTargetName(Plugin), Plugin->GetContentDir(), LocalizationTargetRootDirectoryOverride);
}

FString GetLocalizationTargetDirectory(const FString& LocalizationTargetName, const FString& PluginContentDirectory)
{
	return FPaths::Combine(PluginContentDirectory, TEXT("Localization"), LocalizationTargetName);
}

FString GetLocalizationTargetDirectory(const FString& LocalizationTargetName, const FString& PluginContentDirectory, const FString& LocalizationTargetRootDirectoryOverride)
{
	return LocalizationTargetRootDirectoryOverride.IsEmpty()
		? GetLocalizationTargetDirectory(LocalizationTargetName, PluginContentDirectory)
		: FPaths::Combine(LocalizationTargetRootDirectoryOverride, LocalizationTargetName);
}

FString GetLocalizationTargetUGCLocFile(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory)
{
	return FPaths::Combine(LocalizationTargetDirectory, LocalizationTargetName + TEXT(".ugcloc"));
}

FString GetLocalizationTargetPOFile(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory, const FString& Culture)
{
	return FPaths::Combine(LocalizationTargetDirectory, Culture, LocalizationTargetName + TEXT(".po"));
}

void PreWriteFileWithSCC(const FString& Filename, const bool bUseSourceControl)
{
	if (bUseSourceControl && USourceControlHelpers::IsAvailable())
	{
		// If the file already already exists, then check it out before writing to it
		// We also consider an add here, as the file may have been added on disk prior to running an export (eg, when running automation that downloads the files from elsewhere)
		if (FPaths::FileExists(Filename))
		{
			if (USourceControlHelpers::CheckOutOrAddFile(Filename, /*bSilent*/true))
			{
				// Make sure the file is actually writable, as adding a read-only file to source control may leave it read-only
				if (IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					PlatformFile.IsReadOnly(*Filename))
				{
					if (!PlatformFile.SetReadOnly(*Filename, false))
					{
						UE_LOGF(LogUGCLocalization, Error, "Failed to make file '%ls' writable", *Filename);
					}
				}
			}
			else
			{
				UE_LOGF(LogUGCLocalization, Error, "Failed to check out or add file '%ls'. %ls", *Filename, *USourceControlHelpers::LastErrorMsg().ToString());
			}
		}
	}
}

void PostWriteFileWithSCC(const FString& Filename, const bool bUseSourceControl)
{
	if (bUseSourceControl && USourceControlHelpers::IsAvailable())
	{
		// If the file didn't exist before then this will add it, otherwise it will do nothing
		if (USourceControlHelpers::CheckOutOrAddFile(Filename, /*bSilent*/true))
		{
			// Discard the checkout if the file has no changes
			USourceControlHelpers::RevertUnchangedFile(Filename, /*bSilent*/true);
		}
		else
		{
			UE_LOGF(LogUGCLocalization, Error, "Failed to check out or add file '%ls'. %ls", *Filename, *USourceControlHelpers::LastErrorMsg().ToString());
		}
	}
}

FLocFileSCCNotifies CreateLocFileNotifies(const bool bUseSourceControl)
{
	TSharedPtr<FLocalizationSCC> SourceControlInfo;
	if (bUseSourceControl)
	{
		SourceControlInfo = FLocalizationSCC::CreateIfSCCEnabled(FLocalizationSCC::EInitSCC::Never);
	}
	return FLocFileSCCNotifies{ SourceControlInfo };
}

void GetLocalizationFileHashes(const FString& BaseDirectory, TMap<FString, uint64>& OutFileHashes)
{
	// Get the list of localization files under the given directory
	TArray<TTuple<FString, uint64>> FilesToHash;
	IFileManager::Get().IterateDirectoryRecursively(*BaseDirectory, [&FilesToHash](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			if (FStringView FileExtension = FPathViews::GetExtension(FilenameOrDirectory);
				FileExtension == TEXTVIEW("po") || FileExtension == TEXTVIEW("ugcloc"))
			{
				FilesToHash.Add(MakeTuple(FilenameOrDirectory, 0));
			}
		}
		return true;
	});

	// Generate their content hashes in parallel
	ParallelFor(TEXT("UserGeneratedContentLocalization.GetLocalizationFileHashes"), FilesToHash.Num(), 1, [&FilesToHash](int32 Index)
	{
		TTuple<FString, uint64>& FileToHashPair = FilesToHash[Index];

		FileToHashPair.Value = FPathViews::GetExtension(FileToHashPair.Key) == TEXTVIEW("po")
			? UE::Localization::FileUtil::PortableObjectHash(FileToHashPair.Key)
			: UE::Localization::FileUtil::TextHash(FileToHashPair.Key);
	});

	// Append the content hashes to the result
	OutFileHashes.Reserve(OutFileHashes.Num() + FilesToHash.Num());
	for (TTuple<FString, uint64>& FileToHashPair : FilesToHash)
	{
		OutFileHashes.Add(MoveTemp(FileToHashPair.Key), MoveTemp(FileToHashPair.Value));
	}
}

bool ExportLocalization(TArrayView<const TSharedRef<IPlugin>> Plugins, const FExportLocalizationOptions& ExportOptions, TFunctionRef<int32(const FString&, FString&)> CommandletExecutor)
{
	if (ExportOptions.UGCLocDescriptor.NativeCulture.IsEmpty())
	{
		UE_LOGF(LogUGCLocalization, Error, "Localization export options did not have a 'NativeCulture' set");
		return false;
	}

	IFileManager& FileManager = IFileManager::Get();

	// Create a scratch directory for the temporary localization data
	const FString RootLocalizationScratchDirectory = GetLocalizationScratchDirectory();
	FileManager.MakeDirectory(*RootLocalizationScratchDirectory, /*bTree*/true);
	ON_SCOPE_EXIT
	{
		if (ExportOptions.bAutoCleanup)
		{
			// Delete the entire scratch directory
			FileManager.DeleteDirectory(*RootLocalizationScratchDirectory, /*bRequireExists*/false, /*bTree*/true);
		}
	};

	// Make sure we're also exporting localization for the native culture
	TArray<FString> CulturesToGenerate = ExportOptions.UGCLocDescriptor.CulturesToGenerate;
	CulturesToGenerate.AddUnique(ExportOptions.UGCLocDescriptor.NativeCulture);

	// Localization data stored per-plugin
	TArray<FString, TInlineAllocator<1>> GatherConfigFilenames;
	TMap<FString, TMap<FString, uint64>> PerTargetLocalizationFileHashes;
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		const FString PluginLocalizationTargetName = GetLocalizationTargetName(Plugin);
		const FString PluginLocalizationScratchDirectory = GetLocalizationScratchDirectory(PluginLocalizationTargetName);
		const FString PluginLocalizationTargetDirectory = GetLocalizationTargetDirectory(PluginLocalizationTargetName, Plugin->GetContentDir(), ExportOptions.LocalizationTargetRootDirectoryOverride);

		// Track the source file hashes when the export started, so that we can detect post-export whether the files have actually changed
		TMap<FString, uint64>& LocalizationFileHashes = PerTargetLocalizationFileHashes.FindOrAdd(PluginLocalizationTargetName);
		GetLocalizationFileHashes(PluginLocalizationTargetDirectory, LocalizationFileHashes);

		// Seed the scratch directory with the current localization files for this plugin, so that the loc gather will import and preserve any existing translation data
		for (const TTuple<FString, uint64>& LocalizationFileHashPair : LocalizationFileHashes)
		{
			const FString& SourceFilename = LocalizationFileHashPair.Key;
			FString DestinationFilename = SourceFilename;
			if (DestinationFilename.ReplaceInline(*PluginLocalizationTargetDirectory, *PluginLocalizationScratchDirectory) > 0)
			{
				if (FileManager.Copy(*DestinationFilename, *SourceFilename) == COPY_OK)
				{
					// Preserve the source timestamp, as MergeLocalization relies on this
					FileManager.SetTimeStamp(*DestinationFilename, FileManager.GetTimeStamp(*SourceFilename));
					UE_LOGF(LogUGCLocalization, Display, "Imported existing file for '%ls': %ls", *PluginLocalizationTargetName, *SourceFilename);
				}
				else
				{
					UE_LOGF(LogUGCLocalization, Error, "Failed to import existing file for '%ls': %ls", *PluginLocalizationTargetName, *SourceFilename);
					return false;
				}
			}
		}
		if (!ExportOptions.LocalizationTargetRootDirectoryOverride.IsEmpty() && ExportOptions.MergeProjectDataWithRootDirectoryOverrideData.IsSet())
		{
			const FString PluginLocalizationSourceDirectory = GetLocalizationTargetDirectory(PluginLocalizationTargetName, Plugin->GetContentDir());
			if (!MergeLocalization(PluginLocalizationTargetName, PluginLocalizationSourceDirectory, PluginLocalizationTargetName, PluginLocalizationScratchDirectory, ExportOptions.MergeProjectDataWithRootDirectoryOverrideData.GetValue(), /*bOnlyIfSourceFileNewer*/true, /*bUseSourceControl*/false))
			{
				return false;
			}
		}

		// Build the gather config
		{
			// Build up a basic localization config that will do the following:
			//  1) Gather source/assets in the current plugin
			//  2) Import any existing PO file data
			//  3) Export new PO file data

			FLocalizationConfigurationScript GatherConfig;
			int32 GatherStepIndex = 0;

			// Common
			{
				FConfigSection ConfigSection;

				ConfigSection.Add(TEXT("SourcePath"), FPaths::ConvertRelativePathToFull(PluginLocalizationScratchDirectory));
				ConfigSection.Add(TEXT("DestinationPath"), FPaths::ConvertRelativePathToFull(PluginLocalizationScratchDirectory));

				ConfigSection.Add(TEXT("ManifestName"), FString::Printf(TEXT("%s.manifest"), *PluginLocalizationTargetName));
				ConfigSection.Add(TEXT("ArchiveName"), FString::Printf(TEXT("%s.archive"), *PluginLocalizationTargetName));
				ConfigSection.Add(TEXT("PortableObjectName"), FString::Printf(TEXT("%s.po"), *PluginLocalizationTargetName));

				ConfigSection.Add(TEXT("GatheredSourceBasePath"), FPaths::ConvertRelativePathToFull(Plugin->GetBaseDir()));

				ConfigSection.Add(TEXT("CopyrightNotice"), ExportOptions.CopyrightNotice);

				ConfigSection.Add(TEXT("NativeCulture"), ExportOptions.UGCLocDescriptor.NativeCulture);
				for (const FString& CultureToGenerate : CulturesToGenerate)
				{
					ConfigSection.Add(TEXT("CulturesToGenerate"), *CultureToGenerate);
				}
				
				GatherConfig.AddCommonSettings(MoveTemp(ConfigSection));
			}

			// Gather source
			if (ExportOptions.bGatherSource)
			{
				const FString PluginConfigDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Config")));
				const FString PluginSourceDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Source")));

				TArray<FString, TInlineAllocator<2>> SearchDirectoryPaths;
				if (FPaths::DirectoryExists(PluginConfigDir))
				{
					SearchDirectoryPaths.Add(PluginConfigDir);
				}
				if (FPaths::DirectoryExists(PluginSourceDir))
				{
					SearchDirectoryPaths.Add(PluginSourceDir);
				}

				// Only gather from source if there's valid paths to gather from, as otherwise the commandlet will error
				if (SearchDirectoryPaths.Num() > 0)
				{
					FConfigSection ConfigSection;
					ConfigSection.Add(TEXT("CommandletClass"), TEXT("GatherTextFromSource"));

					ConfigSection.Add(TEXT("FileNameFilters"), TEXT("*.h"));
					ConfigSection.Add(TEXT("FileNameFilters"), TEXT("*.cpp"));
					ConfigSection.Add(TEXT("FileNameFilters"), TEXT("*.inl"));
					ConfigSection.Add(TEXT("FileNameFilters"), TEXT("*.ini"));

					for (const FString& SearchDirectoryPath : SearchDirectoryPaths)
					{
						ConfigSection.Add(TEXT("SearchDirectoryPaths"), SearchDirectoryPath);
					}
					
					GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
				}
			}

			// Gather assets
			if (ExportOptions.bGatherAssets && Plugin->CanContainContent())
			{
				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("GatherTextFromAssets"));

				ConfigSection.Add(TEXT("PackageFileNameFilters"), TEXT("*.uasset"));
				ConfigSection.Add(TEXT("PackageFileNameFilters"), TEXT("*.umap"));

				ConfigSection.Add(TEXT("IncludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetContentDir(), TEXT("*"))));

				ConfigSection.Add(TEXT("ExcludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetContentDir(), TEXT("Localization"), TEXT("*"))));
				ConfigSection.Add(TEXT("ExcludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetContentDir(), TEXT("L10N"), TEXT("*"))));

				if (const FString* CollectionFilter = ExportOptions.PluginNameToCollectionNameFilter.Find(Plugin->GetName()))
				{
					ConfigSection.Add(TEXT("CollectionFilters"), *CollectionFilter);
				}
				
				ConfigSection.Add(TEXT("SearchAllAssets"), TEXT("false"));
				ConfigSection.Add(TEXT("ApplyRedirectorsToCollections"), TEXT("false"));

				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Gather Verse
			if (ExportOptions.bGatherVerse && Plugin->CanContainVerse())
			{
				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("GatherTextFromVerse"));

				ConfigSection.Add(TEXT("IncludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetBaseDir(), TEXT("*"))));

				ConfigSection.Add(TEXT("ExcludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetContentDir(), TEXT("Localization"), TEXT("*"))));
				ConfigSection.Add(TEXT("ExcludePathFilters"), FPaths::ConvertRelativePathToFull(FPaths::Combine(Plugin->GetContentDir(), TEXT("L10N"), TEXT("*"))));
				
				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Generate manifest
			{
				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("GenerateGatherManifest"));
				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Generate archive
			{
				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("GenerateGatherArchive"));
				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Import PO
			{
				// Read the UGC localization descriptor settings that were used to generate this localization data, as we should import against those
				FUserGeneratedContentLocalizationDescriptor UGCLocDescriptorForImport;
				{
					const FString UGCLocFilename = GetLocalizationTargetUGCLocFile(PluginLocalizationTargetName, PluginLocalizationTargetDirectory);
					if (!UGCLocDescriptorForImport.FromJsonFile(*UGCLocFilename))
					{
						UGCLocDescriptorForImport = ExportOptions.UGCLocDescriptor;
					}
				}

				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("InternationalizationExport"));

				ConfigSection.Add(TEXT("bImportLoc"), TEXT("true"));

				ConfigSection.Add(TEXT("POFormat"), StaticEnum<EPortableObjectFormat>()->GetNameStringByValue((int64)UGCLocDescriptorForImport.PoFormat));

				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Export PO
			{
				FConfigSection ConfigSection;
				ConfigSection.Add(TEXT("CommandletClass"), TEXT("InternationalizationExport"));

				ConfigSection.Add(TEXT("bExportLoc"), TEXT("true"));

				ConfigSection.Add(TEXT("POFormat"), StaticEnum<EPortableObjectFormat>()->GetNameStringByValue((int64)ExportOptions.UGCLocDescriptor.PoFormat));

				ConfigSection.Add(TEXT("ShouldPersistCommentsOnExport"), TEXT("true"));

				GatherConfig.AddGatherTextStep(GatherStepIndex++, MoveTemp(ConfigSection));
			}

			// Write config
			{
				GatherConfig.Dirty = true;

				FString GatherConfigFilename = FPaths::ConvertRelativePathToFull(RootLocalizationScratchDirectory / FString::Printf(TEXT("%s.ini"), *PluginLocalizationTargetName));
				if (GatherConfig.Write(GatherConfigFilename))
				{
					GatherConfigFilenames.Add(MoveTemp(GatherConfigFilename));
				}
				else
				{
					UE_LOGF(LogUGCLocalization, Error, "Failed to write gather config for '%ls': %ls", *PluginLocalizationTargetName, *GatherConfigFilename);
					return false;
				}
			}
		}
	}
	
	// Run the commandlet
	if (GatherConfigFilenames.Num() > 0)
	{
		FString CommandletOutput;
		const int32 ReturnCode = CommandletExecutor(FString::Join(GatherConfigFilenames, TEXT(";")), CommandletOutput);

		// Verify the commandlet finished cleanly
		bool bGatherFailed = true;
		if (ReturnCode == 0)
		{
			bGatherFailed = false;
		}
		else
		{
			// The commandlet can sometimes exit with a non-zero return code for reasons unrelated to the localization export
			// If this happens, check to see whether the GatherText commandlet itself exited with a zero return code
			if (CommandletOutput.Contains(TEXT("GatherText completed with exit code 0"), ESearchCase::CaseSensitive))
			{
				bGatherFailed = false;
				UE_LOGF(LogUGCLocalization, Warning, "Localization commandlet finished with a non-zero exit code, but GatherText finished with a zero exit code. Considering the export a success, but there may be errors or omissions in the exported data.");
			}
		}

		// Log the output and result of the commandlet
		{
			UE_LOGF(LogUGCLocalization, Display, "Localization commandlet finished with exit code %d", ReturnCode);

			if (bGatherFailed || UserGeneratedContentLocalization::bAlwaysExportFullGatherLog)
			{
				TArray<FString> CommandletOutputLines;
				CommandletOutput.ParseIntoArrayLines(CommandletOutputLines);

				for (const FString& CommandletOutputLine : CommandletOutputLines)
				{
					UE_LOGF(LogUGCLocalization, Display, "    %ls", *CommandletOutputLine);
				}
			}
		}

		// If the gather failed then skip the rest of the process
		if (bGatherFailed)
		{
			return false;
		}
	}

	FLocFileSCCNotifies LocFileNotifies = CreateLocFileNotifies(ExportOptions.LocalizationTargetRootDirectoryOverride.IsEmpty());

	LocFileNotifies.BeginParallelTasks(); // Batch the below SCC operations
	ON_SCOPE_EXIT
	{
		LocFileNotifies.EndParallelTasks(); // End the SCC batch
	};

	// Copy any updated PO files back to the plugins and write out the localization settings used to generate them
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		const FString PluginLocalizationTargetName = GetLocalizationTargetName(Plugin);
		const FString PluginLocalizationScratchDirectory = GetLocalizationScratchDirectory(PluginLocalizationTargetName);
		const FString PluginLocalizationTargetDirectory = GetLocalizationTargetDirectory(PluginLocalizationTargetName, Plugin->GetContentDir(), ExportOptions.LocalizationTargetRootDirectoryOverride);

		// Write the UGC localization descriptor settings that were used to generate this localization data
		// That will be needed to handle compilation, but also to handle import correctly if the descriptor settings change later
		if (const FString UGCLocFilename = GetLocalizationTargetUGCLocFile(PluginLocalizationTargetName, PluginLocalizationScratchDirectory);
			!ExportOptions.UGCLocDescriptor.ToJsonFile(*UGCLocFilename))
		{
			UE_LOGF(LogUGCLocalization, Warning, "Failed to write updated .ugcloc file for '%ls': %ls", *PluginLocalizationTargetName, *UGCLocFilename);
		}

		// Track the scratch file hashes now that the export has finished, so that we can detect which files have actually changed from their source
		TMap<FString, uint64> ScratchLocalizationFileHashes;
		GetLocalizationFileHashes(PluginLocalizationScratchDirectory, ScratchLocalizationFileHashes);

		const TMap<FString, uint64>& SourceLocalizationFileHashes = PerTargetLocalizationFileHashes.FindChecked(PluginLocalizationTargetName);
		for (const TTuple<FString, uint64>& LocalizationFileHashPair : ScratchLocalizationFileHashes)
		{
			const FString& SourceFilename = LocalizationFileHashPair.Key;
			FString DestinationFilename = SourceFilename;
			if (DestinationFilename.ReplaceInline(*PluginLocalizationScratchDirectory, *PluginLocalizationTargetDirectory) > 0)
			{
				// Only copy files with modified hashes to avoid source control churn
				if (const uint64* SourceFileHash = SourceLocalizationFileHashes.Find(DestinationFilename);
					!SourceFileHash || *SourceFileHash != LocalizationFileHashPair.Value)
				{
					LocFileNotifies.PreFileWrite(DestinationFilename);
					if (FileManager.Copy(*DestinationFilename, *SourceFilename) == COPY_OK)
					{
						LocFileNotifies.PostFileWrite(DestinationFilename);
						UE_LOGF(LogUGCLocalization, Display, "Updated file for '%ls': %ls", *PluginLocalizationTargetName, *DestinationFilename);
					}
					else
					{
						UE_LOGF(LogUGCLocalization, Warning, "Failed to update file for '%ls': %ls", *PluginLocalizationTargetName, *DestinationFilename);
					}
				}
			}
		}

		if (ExportOptions.bUpdatePluginDescriptor && ExportOptions.LocalizationTargetRootDirectoryOverride.IsEmpty())
		{
			FPluginDescriptor PluginDescriptor = Plugin->GetDescriptor();
			if (!PluginDescriptor.LocalizationTargets.ContainsByPredicate([&PluginLocalizationTargetName](const FLocalizationTargetDescriptor& LocalizationTargetDescriptor) { return LocalizationTargetDescriptor.Name == PluginLocalizationTargetName; }))
			{
				FLocalizationTargetDescriptor& LocalizationTargetDescriptor = PluginDescriptor.LocalizationTargets.AddDefaulted_GetRef();
				LocalizationTargetDescriptor.Name = PluginLocalizationTargetName;
				switch (ExportOptions.LocalizationCategory)
				{
				case ELocalizedTextSourceCategory::Game:
					LocalizationTargetDescriptor.LoadingPolicy = ELocalizationTargetDescriptorLoadingPolicy::Game;
					break;

				case ELocalizedTextSourceCategory::Engine:
					LocalizationTargetDescriptor.LoadingPolicy = ELocalizationTargetDescriptorLoadingPolicy::Always;
					break;

				case ELocalizedTextSourceCategory::Editor:
					LocalizationTargetDescriptor.LoadingPolicy = ELocalizationTargetDescriptorLoadingPolicy::Editor;
					break;

				default:
					checkf(false, TEXT("Unexpected ELocalizedTextSourceCategory!"));
					break;
				}

				FText DescriptorUpdateFailureReason;
				LocFileNotifies.PreFileWrite(Plugin->GetDescriptorFileName());
				if (Plugin->UpdateDescriptor(PluginDescriptor, DescriptorUpdateFailureReason))
				{
					LocFileNotifies.PostFileWrite(Plugin->GetDescriptorFileName());
					UE_LOGF(LogUGCLocalization, Display, "Updated .uplugin file for '%ls'", *PluginLocalizationTargetName);
				}
				else
				{
					UE_LOGF(LogUGCLocalization, Warning, "Failed to update .uplugin file for '%ls': %ls", *PluginLocalizationTargetName, *DescriptorUpdateFailureReason.ToString());
				}
			}
		}

		if (ExportOptions.LocalizationTargetRootDirectoryOverride.IsEmpty())
		{
			LocalizationDelegates::OnLocalizationTargetDataUpdated.Broadcast(PluginLocalizationTargetDirectory);
		}
	}

	return true;
}

bool CompileLocalizationTarget(const FString& LocalizationTargetDirectory, const FLocTextHelper& LocTextHelper)
{
	const FString LocMetaName = FString::Printf(TEXT("%s.locmeta"), *LocTextHelper.GetTargetName());
	const FString LocResName = FString::Printf(TEXT("%s.locres"), *LocTextHelper.GetTargetName());

	// Generate the LocMeta file
	{
		FTextLocalizationMetaDataResource LocMeta;
		if (FTextLocalizationResourceGenerator::GenerateLocMeta(LocTextHelper, LocResName, LocMeta))
		{
			LocMeta.bIsUGC = true;
			if (!LocMeta.SaveToFile(LocalizationTargetDirectory / LocMetaName))
			{
				UE_LOGF(LogUGCLocalization, Error, "Failed to save LocMeta file for '%ls'", *LocTextHelper.GetTargetName());
				return false;
			}
		}
		else
		{
			UE_LOGF(LogUGCLocalization, Error, "Failed to generate LocMeta file for '%ls'", *LocTextHelper.GetTargetName());
			return false;
		}
	}

	{
		std::atomic<bool> bAnySaveFailed{false};
		
		const TArray<FString>& AllCultures = LocTextHelper.GetAllCultures();
		const int32 NumCultures = AllCultures.Num();
		ParallelFor (NumCultures, [&bAnySaveFailed, &AllCultures, &LocTextHelper, &LocalizationTargetDirectory, &LocResName](const int32 Index)
			{
				if (bAnySaveFailed.load(std::memory_order_relaxed))
				{
					return;
				}
				
				const FString& CultureToGenerate = AllCultures[Index];
				
				FTextLocalizationResource LocRes;
				TMap<FName, TSharedRef<FTextLocalizationResource>> PerPlatformLocRes;
				if (FTextLocalizationResourceGenerator::GenerateLocRes(LocTextHelper, CultureToGenerate, EGenerateLocResFlags::None, LocalizationTargetDirectory / CultureToGenerate / LocResName, LocRes, PerPlatformLocRes))
				{
					checkf(PerPlatformLocRes.Num() == 0, TEXT("UGC localization does not support per-platform LocRes!"));
		
					if (!LocRes.SaveToFile(LocalizationTargetDirectory / CultureToGenerate / LocResName))
					{
						UE_LOGF(LogUGCLocalization, Error, "Failed to save LocRes file for '%ls' (culture '%ls')", *LocTextHelper.GetTargetName(), *CultureToGenerate);
						bAnySaveFailed.store(true, std::memory_order_relaxed);
						return;
					}
				}
				else
				{
					UE_LOGF(LogUGCLocalization, Error, "Failed to generate LocRes file for '%ls' (culture '%ls')", *LocTextHelper.GetTargetName(), *CultureToGenerate);
					bAnySaveFailed.store(true, std::memory_order_relaxed);
					return;
				}
			});
		
		if (bAnySaveFailed.load(std::memory_order_relaxed))
		{
			return false;
		}
	}

	LocalizationDelegates::OnLocalizationTargetDataUpdated.Broadcast(LocalizationTargetDirectory);

	return true;
}

bool CompileLocalization(TArrayView<const TSharedRef<IPlugin>> Plugins, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor)
{
	// Localization data is stored per-plugin
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		const FString PluginLocalizationTargetName = GetLocalizationTargetName(Plugin);
		const FString PluginLocalizationTargetDirectory = GetLocalizationTargetDirectory(PluginLocalizationTargetName, Plugin->GetContentDir());
		if (!CompileLocalization(PluginLocalizationTargetName, PluginLocalizationTargetDirectory, PluginLocalizationTargetDirectory, DefaultDescriptor))
		{
			return false;
		}
	}

	return true;
}

bool CompileLocalization(const FString& LocalizationTargetName, const FString& LocalizationTargetInputDirectory, const FString& LocalizationTargetOutputDirectory, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor)
{
	// Load the localization data so that we can compile it
	TSharedPtr<FLocTextHelper> LocTextHelper;
	const ELoadLocalizationResult LoadResult = LoadLocalization(LocalizationTargetName, LocalizationTargetInputDirectory, LocTextHelper, DefaultDescriptor);
	if (LoadResult == ELoadLocalizationResult::NoData)
	{
		// Nothing to do
		return true;
	}
	if (LoadResult == ELoadLocalizationResult::Failed)
	{
		// Failed to load, so can't compile
		return false;
	}
	check(LoadResult == ELoadLocalizationResult::Success);

	return CompileLocalizationTarget(LocalizationTargetOutputDirectory, *LocTextHelper);
}

bool ImportPortableObject(const FString& LocalizationTargetDirectory, const FString& CultureToLoad, const EPortableObjectFormat PoFormat, FLocTextHelper& LocTextHelper)
{
	const bool bIsNativeCulture = CultureToLoad == LocTextHelper.GetNativeCulture();

	const FString POFilename = GetLocalizationTargetPOFile(LocTextHelper.GetTargetName(), LocalizationTargetDirectory, CultureToLoad);

	FString POFileData;
	FPortableObjectFormatDOM POFile;
	if (!FFileHelper::LoadFileToString(POFileData, *POFilename) || !POFile.FromString(POFileData))
	{
		return false;
	}

	// Process each PO entry
	for (auto EntryPairIter = POFile.GetEntriesIterator(); EntryPairIter; ++EntryPairIter)
	{
		const TSharedPtr<FPortableObjectEntry>& POEntry = EntryPairIter->Value;
		if (POEntry->MsgId.IsEmpty() || POEntry->MsgStr.Num() == 0 || POEntry->MsgStr[0].IsEmpty())
		{
			// We ignore the header entry or entries with no translation.
			continue;
		}

		FString Namespace;
		FString Key;
		FString SourceText;
		FString Translation;
		PortableObjectPipeline::ParseBasicPOFileEntry(*POEntry, Namespace, Key, SourceText, Translation, ELocalizedTextCollapseMode::IdenticalTextIdAndSource, PoFormat);

		// PO files don't contain the key meta-data so we can't reconstruct this
		// Key meta-data only exists to force the PO file export an ID that contains both the namespace AND key though, so it doesn't matter if it's lost here as it won't affect the LocRes generation
		TSharedPtr<FLocMetadataObject> KeyMetadataObj = nullptr;

		// Not all formats contain the source string, so if the source is empty then 
		// we'll assume the translation was made against the most up-to-date source
		if (SourceText.IsEmpty())
		{
			if (bIsNativeCulture)
			{
				SourceText = Translation;
			}
			else if (const TSharedPtr<FArchiveEntry> NativeEntry = LocTextHelper.FindTranslation(LocTextHelper.GetNativeCulture(), Namespace, Key, KeyMetadataObj))
			{
				SourceText = NativeEntry->Translation.Text;
			}
		}

		// If this is the native culture then we also add it as source in the manifest
		if (bIsNativeCulture)
		{
			FManifestContext ManifestContext;
			ManifestContext.SourceLocation = POEntry->ReferenceComments.Num() > 0 ? POEntry->ReferenceComments[0] : FString();
			ManifestContext.Key = Key;
			ManifestContext.KeyMetadataObj = KeyMetadataObj;
			LocTextHelper.AddSourceText(Namespace, FLocItem(SourceText), ManifestContext);
		}

		// All cultures add this info as a translation
		LocTextHelper.AddTranslation(CultureToLoad, Namespace, Key, KeyMetadataObj, FLocItem(SourceText), FLocItem(Translation), /*bIsOptional*/false);
	}

	return true;
}

ELoadLocalizationResult LoadLocalization(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory, TSharedPtr<FLocTextHelper>& OutLocTextHelper, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor)
{
	const FString UGCLocFilename = GetLocalizationTargetUGCLocFile(LocalizationTargetName, LocalizationTargetDirectory);
	if (!FPaths::FileExists(UGCLocFilename))
	{
		// Nothing to do
		return ELoadLocalizationResult::NoData;
	}

	// Read the UGC localization descriptor settings that were used to generate this localization data
	FUserGeneratedContentLocalizationDescriptor UGCLocDescriptor;
	if (!UGCLocDescriptor.FromJsonFile(*UGCLocFilename))
	{
		UE_LOGF(LogUGCLocalization, Error, "Failed to load localization descriptor for '%ls'", *LocalizationTargetName);
		return ELoadLocalizationResult::Failed;
	}

	// Validate the loaded settings against the given default
	// This will remove/reset any invalid data
	if (DefaultDescriptor)
	{
		UGCLocDescriptor.Validate(*DefaultDescriptor);
	}

	// Create in-memory versions of the manifest/archives that we will populate below
	OutLocTextHelper = MakeShared<FLocTextHelper>(LocalizationTargetDirectory, FString::Printf(TEXT("%s.manifest"), *LocalizationTargetName), FString::Printf(TEXT("%s.archive"), *LocalizationTargetName), UGCLocDescriptor.NativeCulture, UGCLocDescriptor.CulturesToGenerate, nullptr);
	OutLocTextHelper->LoadAll(ELocTextHelperLoadFlags::Create);

	// Do we actually have any PO files to load?
	// If we don't then consider this to be a NoData result (rather than error), as the user likely deleted the PO files but left the UGC localization descriptor
	
	const TArray<FString>& AllCultures = OutLocTextHelper->GetAllCultures();
	{
		bool bHasPOFiles = false;
		for (const FString& CultureToGenerate : AllCultures)
		{
			if (const FString POFilename = GetLocalizationTargetPOFile(OutLocTextHelper->GetTargetName(), LocalizationTargetDirectory, CultureToGenerate);
				FPaths::FileExists(POFilename))
			{
				bHasPOFiles = true;
				break;
			}
		}
		if (!bHasPOFiles)
		{
			return ELoadLocalizationResult::NoData;
		}
	}
	
	const FString NativeCulture = OutLocTextHelper->GetNativeCulture();
	
	// If the native PO file is missing then we effectively have no localization data, but consider that case a warning
	if (const FString POFilename = GetLocalizationTargetPOFile(OutLocTextHelper->GetTargetName(), LocalizationTargetDirectory, NativeCulture);
		!FPaths::FileExists(POFilename))
	{
		UE_LOGF(LogUGCLocalization, Warning, "Missing PO file for '%ls' (culture '%ls')", *OutLocTextHelper->GetTargetName(), *NativeCulture);
		return ELoadLocalizationResult::NoData;
	}

	// Import each PO file data, as we'll use it to generate the LocRes (via FLocTextHelper)
	// We always process the native culture first as it's also used to populate the manifest with the source texts
	if (!ImportPortableObject(LocalizationTargetDirectory, OutLocTextHelper->GetNativeCulture(), UGCLocDescriptor.PoFormat, *OutLocTextHelper))
	{
		UE_LOGF(LogUGCLocalization, Error, "Failed to load PO file for '%ls' (culture '%ls')", *OutLocTextHelper->GetTargetName(), *NativeCulture);
		return ELoadLocalizationResult::Failed;
	}
	
	{
		std::atomic<bool> bAnyImportFailed{false};
		const FString& OutLocTargetName = OutLocTextHelper->GetTargetName();
		ParallelFor(AllCultures.Num(), [&bAnyImportFailed, &AllCultures, &NativeCulture, &OutLocTargetName, &OutLocTextHelper, &LocalizationTargetDirectory, &UGCLocDescriptor](const int32 Index)
			{	
				// this is a simple early-out flag so 'relaxed' is sufficient
				if (bAnyImportFailed.load(std::memory_order_relaxed))
				{
					return;
				}
				
				const FString& Culture = AllCultures[Index];
				if (Culture == NativeCulture)
				{
					return;
				}
				
				// Quick existence check (fast)
				const FString POFilename = GetLocalizationTargetPOFile(OutLocTargetName, LocalizationTargetDirectory, Culture);
				if (!FPaths::FileExists(POFilename))
				{
					UE_LOGF(LogUGCLocalization, Warning, "Missing PO file for '%ls' (culture '%ls')", *OutLocTargetName, *Culture);
					return;
				}
				
				if (!ImportPortableObject(LocalizationTargetDirectory, Culture, UGCLocDescriptor.PoFormat, *OutLocTextHelper))
				{
					UE_LOGF(LogUGCLocalization, Error, "Failed to load PO file for '%ls' (culture '%ls')", *OutLocTargetName, *Culture);
					bAnyImportFailed.store(true, std::memory_order_relaxed);
				}
			});
		
		if (bAnyImportFailed.load(std::memory_order_relaxed))
		{
			return ELoadLocalizationResult::Failed;
		}
	}
	return ELoadLocalizationResult::Success;
}

bool MergeLocalization(const FString& SourceLocalizationTargetName, const FString& SourceLocalizationTargetDirectory, const FString& DestLocalizationTargetName, const FString& DestLocalizationTargetDirectory, const EMergeLocalizationMode MergeMode, const bool bOnlyIfSourceFileNewer, const bool bUseSourceControl)
{
	TMap<FString, uint64> SourceLocalizationFileHashes;
	GetLocalizationFileHashes(SourceLocalizationTargetDirectory, SourceLocalizationFileHashes);

	if (SourceLocalizationFileHashes.IsEmpty())
	{
		// Nothing to do
		return true;
	}

	TMap<FString, uint64> DestLocalizationFileHashes;
	GetLocalizationFileHashes(DestLocalizationTargetDirectory, DestLocalizationFileHashes);

	IFileManager& FileManager = IFileManager::Get();
	FLocFileSCCNotifies LocFileNotifies = CreateLocFileNotifies(bUseSourceControl);

	auto SourceFilenameToDestFilename = [&SourceLocalizationTargetDirectory, &DestLocalizationTargetDirectory, SourcePOFilename = SourceLocalizationTargetName + TEXT(".po"), DestPOFilename = DestLocalizationTargetName + TEXT(".po")](const FString& SourceFilename)
	{
		FString DestFilename = SourceFilename;
		DestFilename.ReplaceInline(*SourceLocalizationTargetDirectory, *DestLocalizationTargetDirectory);
		DestFilename.ReplaceInline(*SourcePOFilename, *DestPOFilename);
		return DestFilename;
	};

	auto LoadPOFile = [](const FString& POFilename, FPortableObjectFormatDOM& OutPOFile)
	{
		FString POFileData;
		if (!FFileHelper::LoadFileToString(POFileData, *POFilename) || !OutPOFile.FromString(POFileData))
		{
			UE_LOGF(LogUGCLocalization, Error, "Merge failed to load '%ls'", *POFilename);
			return false;
		}
		return true;
	};

	auto SavePOFile = [&LocFileNotifies](const FString& POFilename, FPortableObjectFormatDOM& POFile)
	{
		FString POFileData;
		POFile.ToString(POFileData);

		LocFileNotifies.PreFileWrite(POFilename);
		if (FFileHelper::SaveStringToFile(POFileData, *POFilename, FFileHelper::EEncodingOptions::ForceUTF8))
		{
			LocFileNotifies.PostFileWrite(POFilename);
			return true;
		}
		else
		{
			UE_LOGF(LogUGCLocalization, Error, "Merge failed to save '%ls'", *POFilename);
			return false;
		}
	};

	struct FPOFileToMerge
	{
		FString SourceFilename;
		FString DestFilename;
		bool bMergeFailed = false;
	};

	LocFileNotifies.BeginParallelTasks(); // Batch the below SCC operations
	ON_SCOPE_EXIT
	{
		LocFileNotifies.EndParallelTasks(); // End the SCC batch
	};

	TArray<FPOFileToMerge> POFilesToMerge;
	for (const TTuple<FString, uint64>& SourceLocalizationFileHashPair : SourceLocalizationFileHashes)
	{
		const FString& SourceFilename = SourceLocalizationFileHashPair.Key;
		const FString DestFilename = SourceFilenameToDestFilename(SourceFilename);

		if (const uint64* DestLocalizationFileHash = DestLocalizationFileHashes.Find(DestFilename))
		{
			if (SourceLocalizationFileHashPair.Value == *DestLocalizationFileHash)
			{
				// File hash is identical; nothing to merge
				UE_LOGF(LogUGCLocalization, Log, "Merge skipped '%ls' as it is identical to '%ls'", *SourceFilename, *DestFilename);
				continue;
			}
			
			if (bOnlyIfSourceFileNewer)
			{
				const FDateTime SourceTimestamp = FileManager.GetTimeStamp(*SourceFilename);
				const FDateTime DestTimestamp = FileManager.GetTimeStamp(*DestFilename);
				if (SourceTimestamp <= DestTimestamp)
				{
					// Source file is older; nothing to merge
					UE_LOGF(LogUGCLocalization, Log, "Merge skipped '%ls' as it is older than '%ls'", *SourceFilename, *DestFilename);
					continue;
				}
			}
			
			if (FPathViews::GetExtension(SourceFilename) == TEXTVIEW("po"))
			{
				// Merge source into dest
				POFilesToMerge.Add(FPOFileToMerge{ SourceFilename, DestFilename });
			}
		}
		else
		{
			// File doesn't exist at dest; just copy directly from source
			LocFileNotifies.PreFileWrite(DestFilename);
			if (FileManager.Copy(*DestFilename, *SourceFilename) == COPY_OK)
			{
				LocFileNotifies.PostFileWrite(DestFilename);
				UE_LOGF(LogUGCLocalization, Log, "Merge copied '%ls' to '%ls'", *SourceFilename, *DestFilename);
			}
			else
			{
				UE_LOGF(LogUGCLocalization, Error, "Merge failed to copy '%ls' to '%ls'", *SourceFilename, *DestFilename);
				return false;
			}
		}
	}

	ParallelFor(TEXT("UserGeneratedContentLocalization.MergeLocalization"), POFilesToMerge.Num(), 1, [&POFilesToMerge, &LoadPOFile, &SavePOFile, MergeMode](int32 Index)
	{
		FPOFileToMerge& POFileToMerge = POFilesToMerge[Index];

		FPortableObjectFormatDOM SourcePOFile;
		FPortableObjectFormatDOM DestPOFile;
		if (!LoadPOFile(POFileToMerge.SourceFilename, SourcePOFile) || !LoadPOFile(POFileToMerge.DestFilename, DestPOFile))
		{
			POFileToMerge.bMergeFailed = true;
			return;
		}

		bool bModifiedDestPOFile = false;
		for (auto SourceEntryPairIter = SourcePOFile.GetEntriesIterator(); SourceEntryPairIter; ++SourceEntryPairIter)
		{
			const TSharedPtr<FPortableObjectEntry>& SourcePOEntry = SourceEntryPairIter->Value;
			if (SourcePOEntry->MsgId.IsEmpty() || SourcePOEntry->MsgStr.Num() == 0 || SourcePOEntry->MsgStr[0].IsEmpty())
			{
				// We ignore the header entry or entries with no translation
				continue;
			}

			if (TSharedPtr<FPortableObjectEntry> DestPOEntry = DestPOFile.FindEntry(SourcePOEntry.ToSharedRef()))
			{
				// Replace the entry in dest?
				if (MergeMode == EMergeLocalizationMode::All || DestPOEntry->MsgStr.Num() == 0 || DestPOEntry->MsgStr[0].IsEmpty())
				{
					DestPOEntry->MsgStr = SourcePOEntry->MsgStr;
					bModifiedDestPOFile = true;
				}
			}
			else
			{
				// Add a new entry to dest
				DestPOFile.AddEntry(MakeShared<FPortableObjectEntry>(*SourcePOEntry));
				bModifiedDestPOFile = true;
			}
		}

		if (bModifiedDestPOFile)
		{
			if (!SavePOFile(POFileToMerge.DestFilename, DestPOFile))
			{
				POFileToMerge.bMergeFailed = true;
				return;
			}
		}
	});

	for (FPOFileToMerge& POFileToMerge : POFilesToMerge)
	{
		if (POFileToMerge.bMergeFailed)
		{
			return false;
		}

		UE_LOGF(LogUGCLocalization, Log, "Merge applied '%ls' to '%ls'", *POFileToMerge.SourceFilename, *POFileToMerge.DestFilename);
	}

	return true;
}

void CleanupLocalization(TArrayView<const TSharedRef<IPlugin>> PluginsToClean, const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor, const bool bSilent)
{
	CleanupLocalization(PluginsToClean, {}, DefaultDescriptor, bSilent);
}

void CleanupLocalization(TArrayView<const TSharedRef<IPlugin>> PluginsToClean, TArrayView<const TSharedRef<IPlugin>> PluginsToRemove, const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor, const bool bSilent)
{
	// Make sure we also consider localization for the native culture
	TArray<FString> CulturesToGenerate = DefaultDescriptor.CulturesToGenerate;
	if (!DefaultDescriptor.NativeCulture.IsEmpty())
	{
		CulturesToGenerate.AddUnique(DefaultDescriptor.NativeCulture);
	}

	// Compute the files that will be removed
	TArray<FString> LocalizationFilesToCleanup;
	auto ComputeLocalizationFilesToCleanup = [&LocalizationFilesToCleanup, &CulturesToGenerate](TArrayView<const TSharedRef<IPlugin>> Plugins, bool bRemoveAll)
	{
		for (const TSharedRef<IPlugin>& Plugin : Plugins)
		{
			const FString PluginLocalizationTargetName = GetLocalizationTargetName(Plugin);
			const FString PluginLocalizationTargetDirectory = GetLocalizationTargetDirectory(PluginLocalizationTargetName, Plugin->GetContentDir());

			// Find any leftover PO files to cleanup
			const FString PluginPOFilename = PluginLocalizationTargetName + TEXT(".po");
			IFileManager::Get().IterateDirectory(*PluginLocalizationTargetDirectory, [bRemoveAll, &LocalizationFilesToCleanup, &PluginPOFilename, &CulturesToGenerate](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
			{
				if (bIsDirectory)
				{
					// Note: This looks for PO files rather than the folders, as the folders may just be empty vestiges from a P4 sync without rmdir set
					const FString PluginPOFile = FilenameOrDirectory / PluginPOFilename;
					if (FPaths::FileExists(PluginPOFile))
					{
						const FString LocalizationFolder = FPaths::GetCleanFilename(FilenameOrDirectory);
						const FString CanonicalName = FCulture::GetCanonicalName(LocalizationFolder);
						if (bRemoveAll || !CulturesToGenerate.Contains(CanonicalName))
						{
							LocalizationFilesToCleanup.Add(PluginPOFile);
						}
					}
				}
				return true;
			});

			// If we aren't exporting any cultures, then also cleanup any existing descriptor file
			if (bRemoveAll || CulturesToGenerate.Num() == 0)
			{
				const FString PluginUGCLocFilename = GetLocalizationTargetUGCLocFile(PluginLocalizationTargetName, PluginLocalizationTargetDirectory);
				if (FPaths::FileExists(PluginUGCLocFilename))
				{
					LocalizationFilesToCleanup.Add(PluginUGCLocFilename);
				}
			}
		}
	};
	ComputeLocalizationFilesToCleanup(PluginsToClean, /*bRemoveAll*/false);
	ComputeLocalizationFilesToCleanup(PluginsToRemove, /*bRemoveAll*/true);

	// Remove any files that are no longer needed, asking for confirmation when bSilent=false
	if (LocalizationFilesToCleanup.Num() > 0)
	{
		auto GetCleanupLocalizationMessage = [&LocalizationFilesToCleanup]()
		{
			FTextBuilder CleanupLocalizationMessageBuilder;
			CleanupLocalizationMessageBuilder.AppendLine(LOCTEXT("CleanupLocalization.Message", "Would you like to cleanup the following localization data?"));
			for (const FString& LeftoverPOFile : LocalizationFilesToCleanup)
			{
				CleanupLocalizationMessageBuilder.AppendLineFormat(LOCTEXT("CleanupLocalization.MessageLine", "    \u2022 {0}"), FText::AsCultureInvariant(LeftoverPOFile));
			}
			return CleanupLocalizationMessageBuilder.ToText();
		};

		if (bSilent || FMessageDialog::Open(EAppMsgType::YesNo, GetCleanupLocalizationMessage(), LOCTEXT("CleanupLocalization.Title", "Cleanup localization data?")) == EAppReturnType::Yes)
		{
			// Cleanup the files
			if (USourceControlHelpers::IsEnabled())
			{
				USourceControlHelpers::MarkFilesForDelete(LocalizationFilesToCleanup);
			}
			else
			{
				for (const FString& LocalizationFileToCleanup : LocalizationFilesToCleanup)
				{
					IFileManager::Get().Delete(*LocalizationFileToCleanup);
				}
			}

			// Cleanup the folders containing those files (will do nothing if the folder isn't actually empty)
			for (const FString& LocalizationFileToCleanup : LocalizationFilesToCleanup)
			{
				const FString LocalizationPathToCleanup = FPaths::GetPath(LocalizationFileToCleanup);
				IFileManager::Get().DeleteDirectory(*LocalizationPathToCleanup);
			}

			// Remove any leftover localization references from the plugins
			auto RemovePluginLocalizationReferences = [](TArrayView<const TSharedRef<IPlugin>> Plugins)
			{
				FLocFileSCCNotifies LocFileNotifies = CreateLocFileNotifies();

				LocFileNotifies.BeginParallelTasks(); // Batch the below SCC operations
				ON_SCOPE_EXIT
				{
					LocFileNotifies.EndParallelTasks(); // End the SCC batch
				};

				for (const TSharedRef<IPlugin>& Plugin : Plugins)
				{
					const FString PluginLocalizationTargetName = GetLocalizationTargetName(Plugin);

					FPluginDescriptor PluginDescriptor = Plugin->GetDescriptor();
					if (PluginDescriptor.LocalizationTargets.RemoveAll([&PluginLocalizationTargetName](const FLocalizationTargetDescriptor& LocalizationTargetDescriptor) { return LocalizationTargetDescriptor.Name == PluginLocalizationTargetName; }) > 0)
					{
						FText DescriptorUpdateFailureReason;
						LocFileNotifies.PreFileWrite(Plugin->GetDescriptorFileName());
						if (Plugin->UpdateDescriptor(PluginDescriptor, DescriptorUpdateFailureReason))
						{
							LocFileNotifies.PostFileWrite(Plugin->GetDescriptorFileName());
						}
					}
				}
			};
			if (CulturesToGenerate.Num() == 0)
			{
				// If we aren't exporting any cultures, then also cleanup any plugin references to the localization data
				RemovePluginLocalizationReferences(PluginsToClean);
			}
			RemovePluginLocalizationReferences(PluginsToRemove);
		}
	}
}

}

#undef LOCTEXT_NAMESPACE
