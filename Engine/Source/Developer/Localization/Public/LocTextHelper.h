// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectMacros.h"

#include "LocTextHelper.generated.h"

#define UE_API LOCALIZATION_API

class FArchiveEntry;
class FInternationalizationArchive;
class FLocMetadataObject;
class FText;

/** Flags controlling the behavior used when loading manifests and archives into FLocTextHelper */
enum class ELocTextHelperLoadFlags : uint8
{
	/** Attempt to load an existing file, or fail is none is present */
	Load = 1<<0,
	/** Attempt to create a new file, potentially replacing an existing file */
	Create = 1<<1,
	/** Attempt to load an existing file, or create a new file if none is present */
	LoadOrCreate = Load | Create,
};
ENUM_CLASS_FLAGS(ELocTextHelperLoadFlags);

/** What kind of "source" should we use when looking up translations for export? */
enum class ELocTextExportSourceMethod : uint8
{
	/** Use the source text */
	SourceText,
	/** Use the native text */
	NativeText,
};

/** How should we split platform specific localization data? */
UENUM()
enum class ELocTextPlatformSplitMode : uint8
{
	/** Don't split platform specific localization data */
	None,
	/** Split platform specific localization data for confidential platforms only */
	Confidential,
	/** Split platform specific localization data for all platforms */
	All,
};

/**
 * Interface for the loc file notify API.
 * This can be used to integrate with services like source control.
 */
class ILocFileNotifies
{
public:
	/** Virtual destructor */
	virtual ~ILocFileNotifies() = default;

	/** Start a block of parallel tasks */
	virtual void BeginParallelTasks() = 0;

	/** Stop a block of parallel tasks */
	virtual void EndParallelTasks() = 0;

	/** Called prior to reading the given file on disk */
	virtual void PreFileRead(const FString& InFilename) = 0;

	/** Called after reading the given file from disk */
	virtual void PostFileRead(const FString& InFilename) = 0;

	/** Called prior to writing the given file to disk */
	virtual void PreFileWrite(const FString& InFilename) = 0;

	/** Called after writing the given file to disk */
	virtual void PostFileWrite(const FString& InFilename) = 0;
};

/**
 * Utilities to get manage platforms to split.
 */
struct FLocTextPlatformSplitUtils
{
	/** True if the given split mode is actually splitting localization data */
	static UE_API bool ShouldSplitPlatformData(const ELocTextPlatformSplitMode& InSplitMode);

	/** Get the platforms names that should be split, based on the given split mode */
	static UE_API const TArray<FString>& GetPlatformsToSplit(const ELocTextPlatformSplitMode& InSplitMode);
};
/** An enum representing all the formats the conflict report can be output in. */
enum class EConflictReportFormat : uint8
{
	None,
	Txt,
	CSV
};

/**
 * Class that tracks any conflicts that occur when gathering source text entries.
 */
class FLocTextConflicts
{
private:
	/** Internal conflict item. Maps a source identity to all of its conflicts. */
	struct FConflict
	{
	public:
		FConflict(FLocKey InNamespace, FLocKey InKey, TSharedPtr<FLocMetadataObject> InKeyMetadataObj)
			: Namespace(MoveTemp(InNamespace))
			, Key(MoveTemp(InKey))
			, KeyMetadataObj(MoveTemp(InKeyMetadataObj))
		{
		}

		void Add(const FLocItem& Source, const FString& SourceLocation)
		{
			EntriesBySourceLocation.AddUnique(SourceLocation, Source);
		}

		const FLocKey Namespace;
		const FLocKey Key;
		TSharedPtr<FLocMetadataObject> KeyMetadataObj;

		TMultiMap<FString, FLocItem> EntriesBySourceLocation;
	};

	typedef TMultiMap<FLocKey, TSharedRef<FConflict>> FConflictMap;

public:
	FLocTextConflicts() {}

	/**
	 * Add a new conflict entry.
	 *
	 * @param InNamespace			The namespace of the entry.
	 * @param InKey					The key/identifier of the entry.
	 * @param InKeyMetadata			Entry Metadata keys.
	 * @param InSource				The source info for the conflict.
	 * @param InSourceLocation		The source location of the conflict.
	 */
	UE_API void AddConflict(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadata, const FLocItem& InSource, const FString& InSourceLocation);
	
	/**
	 * Convert the conflicts to a string format that can be easily saved as a .txt report summary.
	 */
	UE_API FString GetConflictReport() const;
	/**
	 * Convert the conflicts to a string format that can be easily saved as a .csv report summary.
	 */
	UE_API FString GetConflictReportAsCSV() const;

private:
	FLocTextConflicts(const FLocTextConflicts&) = delete;
	FLocTextConflicts& operator=(const FLocTextConflicts&) = delete;

	/**
	 * Find an existing conflict entry.
	 *
	 * @param InNamespace			The namespace of the entry.
	 * @param InKey					The key/identifier of the entry.
	 * @param InKeyMetadata			Entry Metadata keys.
	 *
	 * @return The entry, or null if there is no current entry for the given key.
	 */
	UE_API TSharedPtr<FConflict> FindEntryByKey(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadata) const;

private:
	FConflictMap EntriesByKey;
};

/**
 * Class that manages the word count reporting of the various cultures.
 */
class FLocTextWordCounts
{
public:
	/** Data representing a single word count row */
	struct FRowData
	{
		FDateTime Timestamp;
		int32 SourceWordCount;
		TMap<FString, int32> PerCultureWordCounts;

		FRowData()
			: Timestamp()
			, SourceWordCount(0)
			, PerCultureWordCounts()
		{
		}

		void ResetWordCounts();

		bool IdenticalWordCounts(const FRowData& InOther) const;
	};

	/**
	 * Add a new row and get its data.
	 *
	 * @param OutIndex				Optional value to fill with the index of the new row.
	 *
	 * @return The row data.
	 */
	UE_API FRowData& AddRow(int32* OutIndex = nullptr);

	/**
	 * Get the data for a row from its index.
	 *
	 * @param InIndex				Index of the row to find.
	 *
	 * @return The row data, or null if no row exists with that index.
	 */
	UE_API FRowData* GetRow(const int32 InIndex);
	UE_API const FRowData* GetRow(const int32 InIndex) const;

	/**
	 * @return The number of rows in this report.
	 */
	UE_API int32 GetRowCount() const;

	/**
	 * Trim entries from the report for the cases where the word counts haven't changed between consecutive rows (as ordered by date).
	 */
	UE_API void TrimReport();

	/**
	 * Populate this word count report from a CSV string (clears any existing data).
	  *
	 * @param InCSVString			String containing the CSV data to import.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the CSV string was imported, false otherwise.
	 */
	UE_API bool FromCSV(const FString& InCSVString, FText* OutError = nullptr);

	/**
	 * Write this word count report to a CSV string.
	 *
	 * @return A string containing this report in CSV format.
	 */
	UE_API FString ToCSV();

private:
	/**
	 * Sort all the rows by descending date.
	 */
	void SortRowsByDate();

	/** Constant column headings for the CSV. */
	static const FString ColHeadingDateTime;
	static const FString ColHeadingWordCount;

	/** Array of data for each row. */
	TArray<FRowData> Rows;
};

/**
 * High-level access to the non-compiled localization resources (manifests and archives) in a way that abstracts some of their quirks.
 * Each instance gives access to a single localization target consisting of a single manifest and several archives (a native archive, and one for each foreign culture).
 */
class FLocTextHelper
{
public:
	/**
	 * Construct an empty helper.
	 * @note This kind of helper is only suitable for dealing with manifests, *not* archives.
	 *
	 * @param InLocFileNotifies		Interface for allowing source control integration (may be null).
	 * @param InTargetName			The name of the target we're working with (eg, Game).
	 * @param InPlatformSplitMode	Should we split localization data per-platform?
	 */
	UE_API FLocTextHelper(FString InTargetName, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode = ELocTextPlatformSplitMode::None);

	/**
	 * Construct a helper for the given target information.
	 * @note Nothing is loaded or created at this point.
	 *
	 * @param InTargetPath			Path to the localization target (the root of this path should contain the manifest, and the archives should be under culture code directories).
	 * @param InManifestName		Name given to the manifest file for this target (eg, Game.manifest).
	 * @param InArchiveName			Name given to the archive files for this target (eg, Game.archive).
	 * @param InNativeCulture		Culture code of the native culture (eg, en), or an empty string if the native culture is unknown.
	 * @param InForeignCultures		Array of culture codes for the foreign cultures (the native culture will be removed from this array if present).
	 * @param InLocFileNotifies		Interface for allowing source control integration (may be null).
	 * @param InPlatformSplitMode	Should we split localization data per-platform?
	 */
	UE_API FLocTextHelper(FString InTargetPath, FString InManifestName, FString InArchiveName, FString InNativeCulture, TArray<FString> InForeignCultures, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode = ELocTextPlatformSplitMode::None);

	/**
	 * Initialize helper with already loaded manifest.
	 * @note Suitable for dealing with in-memory manifest (but not limited to).
	 *
	 * @param InLoadedManifest		A pre-loaded manifest for intrusive LocTextHelper initialization.
	 * @param InTargetName			The name of the loc target. Used to infer ManifestName and ArchiveName.
	 * @param InTargetPath			Path to the localization target the pre-loaded manifest refers or should be saved to.
	 * @param InNativeCulture		Culture code of the native culture (eg, en), or an empty string if the native culture is unknown.
	 * @param InForeignCultures		Array of culture codes for the foreign cultures (the native culture will be removed from this array if present).
	 * @param InLocFileNotifies		Interface for allowing source control integration (may be null).
	 * @param InPlatformSplitMode	Should we split localization data per-platform?
	 */
	UE_API FLocTextHelper(TSharedPtr<FInternationalizationManifest> InLoadedManifest, FString InTargetName, FString InTargetPath, FString InNativeCulture, TArray<FString> InForeignCultures, TSharedPtr<ILocFileNotifies> InLocFileNotifies, const ELocTextPlatformSplitMode InPlatformSplitMode = ELocTextPlatformSplitMode::None);

	/**
	 * Are we splitting localization data per-platform?
	 */
	UE_API bool ShouldSplitPlatformData() const;

	/**
	 * How are we splitting localization data per-platform?
	 */
	UE_API ELocTextPlatformSplitMode GetPlatformSplitMode() const;

	/**
	 * Get the platforms names that should be split, based on the active split mode.
	 */
	UE_API const TArray<FString>& GetPlatformsToSplit() const;

	/**
	 * @return The name of the target we're working with.
	 */
	UE_API const FString& GetTargetName() const;

	/**
	 * @return The path to the localization target (the root of this path should contain the manifest, and the archives should be under culture code directories).
	 */
	UE_API const FString& GetTargetPath() const;

	/**
	 * @return The interface that allows source control integration (may be null).
	 */
	UE_API TSharedPtr<ILocFileNotifies> GetLocFileNotifies() const;

	/**
	 * Set the optional copyright notice to insert into files that support it.
	 */
	UE_API void SetCopyrightNotice(const FString& InCopyrightNotice);

	/**
	 * @return Get the optional copyright notice to insert into files that support it.
	 */
	UE_API const FString& GetCopyrightNotice() const;

	/**
	 * @return Get the culture code of the native culture (eg, en), or an empty string if the native culture is unknown.
	 */
	UE_API const FString& GetNativeCulture() const;

	/**
	 * @return Get an array of culture codes for the foreign cultures (does not include the native culture).
	 */
	UE_API const TArray<FString>& GetForeignCultures() const;

	/**
	 * @return Get an array of culture codes for all the cultures (native and foreign).
	 */
	UE_API TArray<FString> GetAllCultures(const bool bSingleCultureMode = false) const;

	/**
	 * Check to see whether we've loaded the manifest.
	 */
	UE_API bool HasManifest() const;

	/**
	 * Attempt to load (or create) the manifest file.
	 *
	 * @param InLoadFlags			Flags controlling whether we should load or create the manifest file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadManifest(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to load (or create) the manifest file from the given file path.
	 *
	 * @param InManifestFilePath	Full file path to load the manifest from.
	 * @param InLoadFlags			Flags controlling whether we should load or create the manifest file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadManifest(const FString& InManifestFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save the manifest file.
	 *
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveManifest(FText* OutError = nullptr) const;

	/**
	 * Attempt to save the manifest file to the given file path.
	 *
	 * @param InManifestFilePath	Full file path to save the manifest to.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveManifest(const FString& InManifestFilePath, FText* OutError = nullptr) const;

	/**
	 * Trim the currently loaded manifest by remove all dependency entries from it.
	 */
	UE_API void TrimManifest();

	/**
	 * Sort the currently loaded manifest into a deterministic order (as if it had been serialized and then deserialized from JSON).
	 */
	UE_API void SortManifest();

	/**
	 * Check to see whether we've loaded the native archive.
	 */
	UE_API bool HasNativeArchive() const;

	/**
	 * Attempt to load (or create) the native archive file.
	 * @note This requires that the native culture was set during construction.
	 * @note The manifest must have been loaded prior to loading archives.
	 *
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadNativeArchive(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to load (or create) the native archive file from the given file path.
	 * @note This requires that the native culture was set during construction.
	 * @note The manifest must have been loaded prior to loading archives.
	 *
	 * @param InArchiveFilePath		Full file path to read the archive from.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadNativeArchive(const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save the native archive file.
	 *
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveNativeArchive(FText* OutError = nullptr) const;

	/**
	 * Attempt to save the native archive file to the given file path.
	 *
	 * @param InArchiveFilePath		Full file path to write the archive as.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveNativeArchive(const FString& InArchiveFilePath, FText* OutError = nullptr) const;

	/**
	 * Check to see whether we've loaded the given foreign archive.
	 * @note This requires that the given culture is in the list of foreign cultures set during construction.
	 */
	UE_API bool HasForeignArchive(const FString& InCulture) const;

	/**
	 * Attempt to load (or create) a foreign archive file.
	 * @note This requires that the given culture is in the list of foreign cultures set during construction.
	 * @note The manifest and native archive (if set) must have been loaded prior to loading foreign archives.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadForeignArchive(const FString& InCulture, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to load (or create) a foreign archive file from the given file path.
	 * @note This requires that the given culture is in the list of foreign cultures set during construction.
	 * @note The manifest and native archive (if set) must have been loaded prior to loading foreign archives.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InArchiveFilePath		Full file path to load the archive from.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadForeignArchive(const FString& InCulture, const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save a foreign archive file.
	 * @note This requires that the given culture is in the list of foreign cultures set during construction.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveForeignArchive(const FString& InCulture, FText* OutError = nullptr) const;

	/**
	 * Attempt to save a foreign archive file to the given file path.
	 * @note This requires that the given culture is in the list of foreign cultures set during construction.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InArchiveFilePath		Full file path to write the archive as.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveForeignArchive(const FString& InCulture, const FString& InArchiveFilePath, FText* OutError = nullptr) const;

	/**
	 * Check to see whether we've loaded the given archive (native or foreign).
	 */
	UE_API bool HasArchive(const FString& InCulture) const;

	/**
	 * Attempt to load (or create) an archive file (native or foreign).
	 * @note This requires that the given culture is in the native culture, or in the list of foreign cultures set during construction.
	 * @note The manifest and native archive (if set) must have been loaded prior to loading foreign archives.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadArchive(const FString& InCulture, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to load (or create) an foreign file (native or foreign) from the given file path.
	 * @note This requires that the given culture is is in the native culture, or in the list of foreign cultures set during construction.
	 * @note The manifest and native archive (if set) must have been loaded prior to loading foreign archives.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InArchiveFilePath		Full file path to load the archive from.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was loaded (or created), false otherwise.
	 */
	UE_API bool LoadArchive(const FString& InCulture, const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save an archive file (native or foreign).
	 * @note This requires that the given culture is is in the native culture, or in the list of foreign cultures set during construction.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveArchive(const FString& InCulture, FText* OutError = nullptr) const;

	/**
	 * Attempt to save an archive file (native or foreign) to the given file path.
	 * @note This requires that the given culture is is in the native culture, or in the list of foreign cultures set during construction.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 * @param InArchiveFilePath		Full file path to write the archive as.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveArchive(const FString& InCulture, const FString& InArchiveFilePath, FText* OutError = nullptr) const;

	/**
	 * Attempt to load (or create) all archive files.
	 * @note The manifest must have been loaded prior to loading archives.
	 *
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the files were loaded (or created), false otherwise.
	 */
	UE_API bool LoadAllArchives(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save all (native and foreign) archive files.
	 *
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveAllArchives(FText* OutError = nullptr) const;

	/**
	 * Trim the given archive by remove any entries that no longer exist in the manifest.
	 *
	 * @param InCulture				Culture code of the archive to load.
	 */
	UE_API void TrimArchive(const FString& InCulture);

	/**
	 * Attempt to load (or create) the manifest and all archive files specified during construction.
	 *
	 * @param InLoadFlags			Flags controlling whether we should load or create the files.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if all files were loaded (or created), false otherwise.
	 */
	UE_API bool LoadAll(const ELocTextHelperLoadFlags InLoadFlags, FText* OutError = nullptr);

	/**
	 * Attempt to save the manifest and all archive files specified during construction.
	 *
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if all files were saved, false otherwise.
	 */
	UE_API bool SaveAll(FText* OutError = nullptr) const;

	/**
	 * Attempt to add a manifest dependency.
	 *
	 * @param InManifestFilePath	Full path to the dependency to load.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was load, false otherwise.
	 */
	UE_API bool AddDependency(const FString& InDependencyFilePath, FText* OutError = nullptr);

	/**
	 * Find an dependency entry using its namespace and key.
	 *
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InSourceText			Optional source text pointer to compare against the source text in the entry.
	 * @param OutDependencyFilePath Optional string to fill with the path to the manifest file that contained the matching dependency.
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	UE_API TSharedPtr<FManifestEntry> FindDependencyEntry(const FLocKey& InNamespace, const FLocKey& InKey, const FString* InSourceText = nullptr, FString* OutDependencyFilePath = nullptr) const;

	/**
	 * Find an existing dependency entry using its namespace and context.
	 *
	 * @param InNamespace			Namespace of the text.
	 * @param InContext				Context information for the text (including its key).
	 * @param OutDependencyFilePath Optional string to fill with the path to the manifest file that contained the matching dependency
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	UE_API TSharedPtr<FManifestEntry> FindDependencyEntry(const FLocKey& InNamespace, const FManifestContext& InContext, FString* OutDependencyFilePath = nullptr) const;

	/**
	 * Add a new source text entry to the manifest.
	 *
	 * @param InNamespace			Namespace of the source text.
	 * @param InSource				Source text to add.
	 * @param InContext				Context information for the source text (including its key).
	 * @param InDescription			Optional description of the source text (for logging).
	 *
	 * @return Returns true if it was added successfully (or if a matching entry already exists), false if a duplicate entry was found with different text (an identity conflict).
	 */
	UE_API bool AddSourceText(const FLocKey& InNamespace, const FLocItem& InSource, const FManifestContext& InContext, const FString* InDescription = nullptr);

	/**
	 * Update an existing source text entry in the manifest.
	 *
	 * @param InOldEntry			Old entry to update.
	 * @param InNewEntry			New entry to set.
	 */
	UE_API void UpdateSourceText(const TSharedRef<FManifestEntry>& InOldEntry, TSharedRef<FManifestEntry>& InNewEntry);

	/**
	 * Find an existing source text entry using its namespace and key.
	 *
	 * @param InNamespace			Namespace of the source text.
	 * @param InKey					Key of the source text.
	 * @param InSourceText			Optional source text pointer to compare against the source text in the entry.
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	UE_API TSharedPtr<FManifestEntry> FindSourceText(const FLocKey& InNamespace, const FLocKey& InKey, const FString* InSourceText = nullptr) const;

	/**
	 * Find an existing source text entry using its namespace and context.
	 *
	 * @param InNamespace			Namespace of the source text.
	 * @param InContext				Context information for the source text (including its key).
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	UE_API TSharedPtr<FManifestEntry> FindSourceText(const FLocKey& InNamespace, const FManifestContext& InContext) const;

	/**
	 * Enumerate all the source texts in the manifest, optionally skipping those entries from a dependent manifest.
	 *
	 * @param InCallback			Function to call for each source text. Returns true to continue enumeration.
	 * @param InCheckDependencies	True to skip entries that can be found in a dependent manifest, false to include them.
	 */
	typedef TFunctionRef<bool(TSharedRef<FManifestEntry>)> FEnumerateSourceTextsFuncPtr;
	UE_API void EnumerateSourceTexts(const FEnumerateSourceTextsFuncPtr& InCallback, const bool InCheckDependencies) const;

	/**
	 * Enumerate in parallel all the source texts in the manifest, optionally skipping those entries from a dependent manifest.
	 * InCallback shall be called from multiple threads so it is the calllers responsibility to guarantee thread safety inside
	 * the callback's scope.
	 *
	 * @param InCallback			Function to call for each source text. Returns true to continue enumeration.
	 * @param InCheckDependencies	True to skip entries that can be found in a dependent manifest, false to include them.
	 */
	UE_API void EnumerateSourceTextsParallel(const FEnumerateSourceTextsFuncPtr& InCallback, const bool InCheckDependencies) const;

	/**
	 * Add a new translation to the given archive.
	 *
	 * @param InCulture				Culture to add the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param InSource				Source text to key against.
	 * @param InTranslation			Translated text to set.
	 * @param InOptional			Is this translation optional?
	 *
	 * @return Returns true if it was added successfully, false otherwise.
	 */
	UE_API bool AddTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation, const bool InOptional);

	/**
	 * Add a new translation to the given archive.
	 *
	 * @param InCulture				Culture to add the translation for.
	 * @param InEntry				Archive entry to add.
	 *
	 * @return Returns true if it was added successfully, false otherwise.
	 */
	UE_API bool AddTranslation(const FString& InCulture, const TSharedRef<FArchiveEntry>& InEntry);

	/**
	 * Update an existing translation in the given archive.
	 *
	 * @param InCulture				Culture to update the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param InSource				Source text to key against.
	 * @param InTranslation			Translated text to set.
	 *
	 * @return Returns true if it was updated successfully, false otherwise.
	 */
	UE_API bool UpdateTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation);

	/**
	 * Update an existing translation in the given archive.
	 *
	 * @param InCulture				Culture to update the translation for.
	 * @param InOldEntry			Old entry to update.
	 * @param InNewEntry			New entry to set.
	 */
	UE_API void UpdateTranslation(const FString& InCulture, const TSharedRef<FArchiveEntry>& InOldEntry, const TSharedRef<FArchiveEntry>& InNewEntry);

	/**
	 * Import a previously exported translation (generated using GetExportText) back into the archive.
	 * This will either update an existing translation, or add a new one if it can't be found.
	 *
	 * @param InCulture				Culture to update the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param InSource				Source text to key against.
	 * @param InTranslation			Translated text to set.
	 * @param InOptional			Is this translation optional?
	 *
	 * @return Returns true if it was imported successfully, false otherwise.
	 */
	UE_API bool ImportTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const FLocItem& InSource, const FLocItem& InTranslation, const bool InOptional);

	/**
	 * Find an existing translation entry from its source text.
	 *
	 * @param InCulture				Culture to find the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	UE_API TSharedPtr<FArchiveEntry> FindTranslation(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj) const;

	/**
	 * Enumerate all the translations for the given culture.
	 * @note This only enumerates translations that have a source in the manifest.
	 *
	 * @param InCulture				Culture to enumerate the translation for.
	 * @param InCallback			Function to call for each translation. Returns true to continue enumeration.
	 * @param InCheckDependencies	True to skip entries that can be found in a dependent manifest, false to include them.
	 */
	typedef TFunctionRef<bool(TSharedRef<FArchiveEntry>)> FEnumerateTranslationsFuncPtr;
	UE_API void EnumerateTranslations(const FString& InCulture, const FEnumerateTranslationsFuncPtr& InCallback, const bool InCheckDependencies) const;

	/**
	 * Given some source text, work out which text should be exported (eg, when exporting to PO).
	 *
	 * @param InCulture				Culture to find the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param InSourceMethod		What kind of "source" should we use when looking up translations?
	 * @param InSource				The raw source text to use as a fallback.
	 * @param OutSource				The source to export.
	 * @param OutTranslation		The translation to export.
	 */
	UE_API void GetExportText(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const ELocTextExportSourceMethod InSourceMethod, const FLocItem& InSource, FLocItem& OutSource, FLocItem& OutTranslation) const;

	/**
	 * Given some source text, work out which text is our current "best" translation (eg, when compiling to LocRes).
	 *
	 * @param InCulture				Culture to find the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param InSourceMethod		What kind of "source" should we use when looking up translations?
	 * @param InSource				The raw source text to use as a fallback.
	 * @param OutTranslation		The translation to use.
	 * @param bSkipSourceCheck		True to skip the source check and just return any matching translation.
	 */
	UE_API void GetRuntimeText(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, const ELocTextExportSourceMethod InSourceMethod, const FLocItem& InSource, FLocItem& OutTranslation, const bool bSkipSourceCheck) const;

	/**
	 * Add a new conflict entry.
	 *
	 * @param InNamespace			The namespace of the entry.
	 * @param InKey					The key/identifier of the entry.
	 * @param InKeyMetadata			Entry Metadata keys.
	 * @param InSource				The source info for the conflict.
	 * @param InSourceLocation		The source location of the conflict.
	 */
	UE_API void AddConflict(const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetadata, const FLocItem& InSource, const FString& InSourceLocation);

	/**
	 * Get a conflict report that can be easily saved as a report summary.
	 */
	UE_API FString GetConflictReport() const;
	

	/**
	 * Save the conflict report summary to disk.
	 *
	 * @param InReportFilePath		Full file path to write the report to.
	 * @param InConflictReportFormat		The format the conflict report should be generated in.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveConflictReport(const FString& InReportFilePath, EConflictReportFormat InConflictReportFormat, FText* OutError = nullptr) const;

	/**
	 * Get a word count report for the current state of the manifest and archives.
	 *
	 * @param InTimestamp			Timestamp to use when adding this entry to the report.
	 * @param InBaseReportFilePath	Optional string containing the file path to a report we should use as a base for this one (loads that data before adding ours to it).
	 *
	 * @return The word count report.
	 */
	UE_API FLocTextWordCounts GetWordCountReport(const FDateTime& InTimestamp, const TCHAR* InBaseReportFilePath = nullptr) const;

	/**
	 * Save the word count report for the current state of the manifest and archives to disk.
	 *
	 * @param InTimestamp			Timestamp to use when adding this entry to the report.
	 * @param InReportFilePath		Full file path to write the report to (also loads any existing report data from this path before adding ours to it).
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	UE_API bool SaveWordCountReport(const FDateTime& InTimestamp, const FString& InReportFilePath, FText* OutError = nullptr) const;

	/**
	 * Sanitize any output from the given string that may cause the build machine to generate erroneous errors.
	 */
	static UE_API FString SanitizeLogOutput(FStringView InString);
	static UE_API FString SanitizeLogOutput(FString&& InString);

	/**
	 * Given a culture, try and find all the keys that the source string should use by checking the manifest.
	 * @note This should only be used to upgrade old non-keyed archive entries when importing legacy data.
	 *
	 * @param InCulture				Culture to find the key for.
	 * @param InNamespace			Namespace of the text.
	 * @param InSource				The source text to find the keys for.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param OutKeys				Array to fill with the found keys.
	 *
	 * @return True if keys were found, false otherwise.
	 */
	UE_API bool FindKeysForLegacyTranslation(const FString& InCulture, const FLocKey& InNamespace, const FString& InSource, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, TArray<FLocKey>& OutKeys) const;

	/**
	 * Given a manifest and (optional) native archive, try and find all the keys that the source string should use by checking the manifest.
	 * @note This should only be used to upgrade old non-keyed archive entries when importing legacy data.
	 *
	 * @param InManifest			The manifest to find the source string in.
	 * @param InNativeArchive		The native archive to test to see if the given source text is really a native translation.
	 * @param InNamespace			Namespace of the text.
	 * @param InSource				The source text to find the keys for.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 * @param OutKeys				Array to fill with the found keys.
	 *
	 * @return True if keys were found, false otherwise.
	 */
	static UE_API bool FindKeysForLegacyTranslation(const TSharedRef<const FInternationalizationManifest>& InManifest, const TSharedPtr<const FInternationalizationArchive>& InNativeArchive, const FLocKey& InNamespace, const FString& InSource, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj, TArray<FLocKey>& OutKeys);

private:
	/** Non-copyable */
	FLocTextHelper(const FLocTextHelper&) = delete;
	FLocTextHelper& operator=(const FLocTextHelper&) = delete;

	/**
	 * Internal implementation of loading (or creating) a manifest.
	 *
	 * @param InManifestFilePath	Full path to the manifest file to load.
	 * @param InLoadFlags			Flags controlling whether we should load or create the manifest file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return The file that was loaded (or created), null otherwise.
	 */
	TSharedPtr<FInternationalizationManifest> LoadManifestImpl(const FString& InManifestFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError);

	/**
	 * Internal implementation of saving a manifest.
	 *
	 * @param InManifest			The manifest file to save.
	 * @param InManifestFilePath	Full path to where the manifest file should be saved.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	bool SaveManifestImpl(const TSharedRef<const FInternationalizationManifest>& InManifest, const FString& InManifestFilePath, FText* OutError) const;

	/**
	 * Internal implementation of loading (or creating) an archive.
	 *
	 * @param InArchiveFilePath		Full path to the archive file to load.
	 * @param InLoadFlags			Flags controlling whether we should load or create the archive file.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return The file that was loaded (or created), null otherwise.
	 */
	TSharedPtr<FInternationalizationArchive> LoadArchiveImpl(const FString& InArchiveFilePath, const ELocTextHelperLoadFlags InLoadFlags, FText* OutError);

	/**
	 * Internal implementation of saving an archive.
	 *
	 * @param InArchive				The archive file to save.
	 * @param InArchiveFilePath		Full path to where the archive file should be saved.
	 * @param OutError				Optional text to be filled when an error occurs.
	 *
	 * @return True if the file was saved, false otherwise.
	 */
	bool SaveArchiveImpl(const TSharedRef<const FInternationalizationArchive>& InArchive, const FString& InArchiveFilePath, FText* OutError) const;

	/**
	 * Find an existing translation entry from its source text.
	 *
	 * @param InCulture				Culture to find the translation for.
	 * @param InNamespace			Namespace of the text.
	 * @param InKey					Key of the text.
	 * @param InKeyMetadataObj		Meta-data associated with the source text key.
	 *
	 * @return The entry, or null if it couldn't be found.
	 */
	TSharedPtr<FArchiveEntry> FindTranslationImpl(const FString& InCulture, const FLocKey& InNamespace, const FLocKey& InKey, const TSharedPtr<FLocMetadataObject> InKeyMetadataObj) const;

	/** How are we splitting localization data per-platform? */
	ELocTextPlatformSplitMode PlatformSplitMode;

	/** The name of the target we're working with */
	FString TargetName;

	/** The path to the localization target (the root of this path should contain the manifest, and the archives should be under culture code directories) */
	FString TargetPath;

	/** Name given to the manifest file for this target (eg, Game.manifest) */
	FString ManifestName;

	/** Name given to the archive files for this target (eg, Game.archive) */
	FString ArchiveName;

	/** Culture code of the native culture (eg, en), or an empty string if the native culture is unknown */
	FString NativeCulture;

	/** Array of culture codes for the foreign cultures (does not include the native culture) */
	TArray<FString> ForeignCultures;

	/** Interface for allowing source control integration (may be null) */
	TSharedPtr<ILocFileNotifies> LocFileNotifies;

	/** Optional copyright notice to insert into files that support it */
	FString CopyrightNotice;

	/** Loaded manifest */
	TSharedPtr<FInternationalizationManifest> Manifest;

	/** Loaded archives */
	TMap<FString, TSharedPtr<FInternationalizationArchive>> Archives;

	/** Loaded dependencies */
	TArray<FString> DependencyPaths;
	TArray<TSharedPtr<FInternationalizationManifest>> Dependencies;

	/** Conflict tracker instance */
	FLocTextConflicts ConflictTracker;
};

#undef UE_API
