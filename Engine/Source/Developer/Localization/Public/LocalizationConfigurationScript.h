// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"

#define UE_API LOCALIZATION_API

class ULocalizationTarget;

struct FLocalizationConfigurationScript : public FConfigFile
{
	UE_API bool WriteWithSCC(const FString& InConfigFilename);

	void AddCommonSettings(FConfigSection&& Section)
	{
		Add(TEXT("CommonSettings"), Section);
	}

	void AddGatherTextStep(const uint32 Index, FConfigSection&& Section)
	{
		Add(FString::Printf( TEXT("GatherTextStep%u"), Index), MoveTemp(Section));
	}
};

namespace LocalizationConfigurationScript
{
	LOCALIZATION_API FString MakePathRelativeForCommandletProcess(const FString& Path, const bool IsUsingProjectFile);

	LOCALIZATION_API FString GetConfigDirectory(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetDataDirectory(const ULocalizationTarget* const Target);

	LOCALIZATION_API TArray<FString> GetConfigPaths(const ULocalizationTarget* const Target);
	LOCALIZATION_API void GenerateAllConfigFiles(const ULocalizationTarget* const Target);

	LOCALIZATION_API TArray<FString> GetOutputFilePaths(const ULocalizationTarget* const Target);

	LOCALIZATION_API FString GetManifestFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetManifestPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FString GetArchiveFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetArchivePath(const ULocalizationTarget* const Target, const FString& CultureName);
	
	LOCALIZATION_API FString GetDefaultPOFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetDefaultPOPath(const ULocalizationTarget* const Target, const FString& CultureName);
	
	LOCALIZATION_API FString GetDefaultDialogueScriptFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetDefaultDialogueScriptPath(const ULocalizationTarget* const Target, const FString& CultureName);
	
	LOCALIZATION_API FString GetLocResFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetLocResPath(const ULocalizationTarget* const Target, const FString& CultureName);

	LOCALIZATION_API FString GetLocMetaFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetLocMetaPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FString GetWordCountCSVFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetWordCountCSVPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FString GetConflictReportFileName(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetConflictReportPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FLocalizationConfigurationScript GenerateGatherTextConfigFile(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetGatherTextConfigPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FLocalizationConfigurationScript GenerateImportTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>(), const TOptional<FString> ImportPathOverride = TOptional<FString>());
	LOCALIZATION_API FString GetImportTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateExportTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>(), const TOptional<FString> ExportPathOverride = TOptional<FString>());
	LOCALIZATION_API FString GetExportTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateImportDialogueScriptConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>(), const TOptional<FString> ImportPathOverride = TOptional<FString>());
	LOCALIZATION_API FString GetImportDialogueScriptConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateExportDialogueScriptConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>(), const TOptional<FString> ExportPathOverride = TOptional<FString>());
	LOCALIZATION_API FString GetExportDialogueScriptConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateImportDialogueConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());
	LOCALIZATION_API FString GetImportDialogueConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateWordCountReportConfigFile(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetWordCountReportConfigPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API FLocalizationConfigurationScript GenerateCompileTextConfigFile(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());
	LOCALIZATION_API FString GetCompileTextConfigPath(const ULocalizationTarget* const Target, const TOptional<FString> CultureName = TOptional<FString>());

	LOCALIZATION_API FLocalizationConfigurationScript GenerateRegenerateResourcesConfigFile(const ULocalizationTarget* const Target);
	LOCALIZATION_API FString GetRegenerateResourcesConfigPath(const ULocalizationTarget* const Target);

	LOCALIZATION_API bool WriteConfigFileWithSCC(const FString& InConfigFilename, FLocalizationConfigurationScript& InConfigScript);
}

#undef UE_API
