// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FString;

namespace LocalizationDelegates
{

/**
 * Delegate called when any on-disk data for the given localization target is updated.
 */
LOCALIZATION_API extern TMulticastDelegate<void(const FString& LocalizationTargetPath)> OnLocalizationTargetDataUpdated;

}
