// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ULocalizationTarget;

class ILocalizationModule : public IModuleInterface
{
public:
	/**
	 * Given a config file, generate a compiled LocRes file for the active culture and use it to update the live-entries in the localization manager.
	 */
	virtual bool HandleRegenLocCommand(const FString& InConfigFilePath) = 0;

	virtual ULocalizationTarget* GetLocalizationTargetByName(FString TargetName, bool bIsEngineTarget) = 0;

	static ILocalizationModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILocalizationModule>("Localization");
	}
};
