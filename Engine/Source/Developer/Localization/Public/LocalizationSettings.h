// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "LocalizationTargetTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "LocalizationSettings.generated.h"

#define UE_API LOCALIZATION_API

class FString;
class ULocalizationTargetSet;
struct FPropertyChangedEvent;

// Class for loading/saving configuration settings and the details view objects needed for localization dashboard functionality.
UCLASS(MinimalAPI, Config=Editor, defaultconfig)
class ULocalizationSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULocalizationSettings(const FObjectInitializer& ObjectInitializer);

private:
	UPROPERTY()
	TObjectPtr<ULocalizationTargetSet> EngineTargetSet;

	UPROPERTY(config)
	TArray<FLocalizationTargetSettings> EngineTargetsSettings;

	UPROPERTY()
	TObjectPtr<ULocalizationTargetSet> GameTargetSet;

	UPROPERTY(config)
	TArray<FLocalizationTargetSettings> GameTargetsSettings;

public:
#if WITH_EDITOR
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	static UE_API ULocalizationTargetSet* GetEngineTargetSet();
	static UE_API ULocalizationTargetSet* GetGameTargetSet();
};

/** Struct containing util functions for getting/setting the SCC settings for the localization dashboard */
struct FLocalizationSourceControlSettings
{
public:
	/** Checks to see whether source control is available based upon the current editor SCC settings. */
	static UE_API bool IsSourceControlAvailable();

	/** Check to see whether we should use SCC when running the localization commandlets. This should be used to optionally pass "-EnableSCC" to the commandlet. */
	static UE_API bool IsSourceControlEnabled();

	/** Check to see whether we should automatically submit changed files after running the commandlet. This should be used to optionally pass "-DisableSCCSubmit" to the commandlet. */
	static UE_API bool IsSourceControlAutoSubmitEnabled();

	/** Set whether we should use SCC when running the localization commandlets. */
	static UE_API void SetSourceControlEnabled(const bool bIsEnabled);

	/** Set whether we should automatically submit changed files after running the commandlet. */
	static UE_API void SetSourceControlAutoSubmitEnabled(const bool bIsEnabled);

private:
	static const FString LocalizationSourceControlSettingsCategoryName;
	static const FString SourceControlEnabledSettingName;
	static const FString SourceControlAutoSubmitEnabledSettingName;
};

#undef UE_API
