// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Async/Mutex.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "LocTextHelper.h"
#include "Templates/SharedPointer.h"

#define UE_API LOCALIZATION_API

class FText;

class FLocalizationSCC
{
public:
	enum class EInitSCC
	{
		Never,
		Always,
		Commandlet,
	};

	UE_API explicit FLocalizationSCC(const EInitSCC InitSCC = EInitSCC::Commandlet);
	UE_API ~FLocalizationSCC();

	/** Create a FLocalizationSCC, but only if a SCC provider is enabled */
	static UE_API TSharedPtr<FLocalizationSCC> CreateIfSCCEnabled(const EInitSCC InitSCC);

	/** Start a block of parallel tasks (may be nested); this will defer check-out requests until the final EndParallelTasks is called */
	UE_API void BeginParallelTasks();
	/** Stop a block of parallel tasks; this will attempt to perform any check-outs deferred during the block */
	UE_API bool EndParallelTasks(FText& OutError);

	/** Attempt to check-out the given file; if called during a parallel tasks block, the file will be made writable and the check-out request deferred */
	UE_API bool CheckOutFile(const FString& InFile, FText& OutError);
	/** Attempt to check-in every file that is currently tracked as checked-out; this cannot be called during a parallel tasks block */
	UE_API bool CheckinFiles(const FText& InChangeDescription, FText& OutError);
	/** Attempt to revert every file that is currently tracked as checked-out; this cannot be called during a parallel tasks block */
	UE_API bool CleanUp(FText& OutError);
	/** Attempt to revert the given file; this cannot be called during a parallel tasks block */
	UE_API bool RevertFile(const FString& InFile, FText& OutError);
	/** Check whether SCC is ready to be used; this cannot be called during a parallel tasks block */
	UE_API bool IsReady(FText& OutError) const;

private:
	/** Set of files that were checked-out from SCC via CheckOutFile */
	TSet<FString> CheckedOutFiles;
	/** Set of files that will be checked-out from SCC via CheckOutFile, but had to be deferred due to running in parallel mode */
	TSet<FString> DeferredCheckedOutFiles;
	/** Mutex protecting CheckedOutFiles and DeferredCheckedOutFiles when ParallelTasksCount > 0 */
	mutable UE::FMutex FilesMutex;
	/** >0 if this we're being used from a parallel tasks block, and need to defer check-out requests */
	std::atomic<uint16> ParallelTasksCount = 0;
	/** True if we called Init on the SCC during construction, and need to call Close during destruction */
	bool bDidInitSCC = false;
};

class FLocFileSCCNotifies : public ILocFileNotifies
{
public:
	FLocFileSCCNotifies(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo)
		: SourceControlInfo(InSourceControlInfo)
	{
	}

	/** Virtual destructor */
	virtual ~FLocFileSCCNotifies() = default;

	//~ ILocFileNotifies interface
	UE_API virtual void BeginParallelTasks() override;
	UE_API virtual void EndParallelTasks() override;
	virtual void PreFileRead(const FString& InFilename) override {}
	virtual void PostFileRead(const FString& InFilename) override {}
	UE_API virtual void PreFileWrite(const FString& InFilename) override;
	UE_API virtual void PostFileWrite(const FString& InFilename) override;

private:
	TSharedPtr<FLocalizationSCC> SourceControlInfo;
};

#undef UE_API
