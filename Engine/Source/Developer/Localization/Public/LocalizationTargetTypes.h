// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "PortableObjectPipeline.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "LocalizationTargetTypes.generated.h"

#define UE_API LOCALIZATION_API

struct FPropertyChangedEvent;

UENUM()
enum class ELocalizationTargetConflictStatus : uint8
{
	/** The status of conflicts in this localization target could not be determined. */
	Unknown,
	/** The are outstanding conflicts present in this localization target. */
	ConflictsPresent,
	/** The localization target is clear of conflicts. */
	Clear,
};

UENUM()
enum class ELocalizationGatherPathRoot : uint8
{
	/** Automatically select Engine or Project based on which set the target belongs to */
	Auto,
	/** Path is relative to the Engine directory */
	Engine,
	/** Path is relative to the Project directory */
	Project,
};

struct FLocalizationGatherPathRootUtil
{
	/** Get the resolved placeholder token of the path root ("%LOCENGINEROOT%" for Engine or "%LOCPROJECTROOT%" for Project; empty for Auto) */
	static UE_API FString GetResolvedPathRootToken(const ELocalizationGatherPathRoot InPathRoot);

	/** Get the path name of the resolved path root (either FPaths::EngineDir() or FPaths::ProjectDir() depending on the setting and target type */
	static UE_API FString GetResolvedPathRoot(const ELocalizationGatherPathRoot InPathRoot, const bool bIsEngineTarget);

	/** Get the display name of the resolved path root (either "Engine/" or "{ProjectName/" depending on the setting and target type */
	static UE_API FText GetResolvedPathRootDisplayName(const ELocalizationGatherPathRoot InPathRoot, const bool bIsEngineTarget);
};

USTRUCT()
struct FGatherTextSearchDirectory
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Path")
	ELocalizationGatherPathRoot PathRoot = ELocalizationGatherPathRoot::Auto;

	UPROPERTY(config, EditAnywhere, Category="Path")
	FString Path;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FGatherTextIncludePath
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Path")
	ELocalizationGatherPathRoot PathRoot = ELocalizationGatherPathRoot::Auto;

	UPROPERTY(config, EditAnywhere, Category="Pattern")
	FString Pattern;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FGatherTextExcludePath
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Path")
	ELocalizationGatherPathRoot PathRoot = ELocalizationGatherPathRoot::Auto;

	UPROPERTY(config, EditAnywhere, Category="Pattern")
	FString Pattern;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FGatherTextFileExtension
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Pattern")
	FString Pattern;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FMetaDataTextKeyPattern
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Pattern")
	FString Pattern;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;

	LOCALIZATION_API static const TArray<FString>& GetPossiblePlaceHolders();
};

USTRUCT()
struct FMetaDataKeyName
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category="Name")
	FString Name;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FMetaDataKeyGatherSpecification
{
	GENERATED_USTRUCT_BODY()

	/*  The metadata key for which values will be gathered as text. */
	UPROPERTY(config, EditAnywhere, Category = "Input")
	FMetaDataKeyName MetaDataKey;

	/* The localization namespace in which the gathered text will be output. */
	UPROPERTY(config, EditAnywhere, Category = "Output")
	FString TextNamespace;

	/* The pattern which will be formatted to form the localization key for the metadata value gathered as text.
	Placeholder - Description
	{FieldPath} - The fully qualified name of the object upon which the metadata resides.
	{MetaDataValue} - The value associated with the metadata key. */
	UPROPERTY(config, EditAnywhere, Category = "Output")
	FMetaDataTextKeyPattern TextKeyPattern;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FGatherTextFromTextFilesConfiguration
{
	GENERATED_USTRUCT_BODY()

	LOCALIZATION_API static const TArray<FGatherTextFileExtension>& GetDefaultTextFileExtensions();

	FGatherTextFromTextFilesConfiguration()
		: IsEnabled(true)
		, FileExtensions(GetDefaultTextFileExtensions())
		, ShouldGatherFromEditorOnlyData(false)
	{
	}

	/* If enabled, text from text files will be gathered according to this configuration. */
	UPROPERTY(config, EditAnywhere, Category = "Execution")
	bool IsEnabled;

	/* The paths of directories to be searched recursively for text files, specified relative to the project's root, which may be parsed for text to gather. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextSearchDirectory> SearchDirectories;

	/* Text files whose paths match these wildcard patterns will be excluded from gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextExcludePath> ExcludePathWildcards;

	/* Text files whose names match these wildcard patterns may be parsed for text to gather. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextFileExtension> FileExtensions;

	/* If enabled, data that is specified as editor-only may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	bool ShouldGatherFromEditorOnlyData;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};


USTRUCT()
struct FGatherTextFromPackagesConfiguration
{
	GENERATED_USTRUCT_BODY()

	LOCALIZATION_API static const TArray<FGatherTextFileExtension>& GetDefaultPackageFileExtensions();

	FGatherTextFromPackagesConfiguration()
		: IsEnabled(true)
		, FileExtensions(GetDefaultPackageFileExtensions())
		, ShouldExcludeDerivedClasses(false)
		, ShouldGatherFromEditorOnlyData(false)
		, SkipGatherCache(false)
	{
		{
			FGatherTextExcludePath L10NPath;
			L10NPath.Pattern = TEXT("Content/L10N/*");
			ExcludePathWildcards.Add(L10NPath);
		}
	}

	/* If enabled, text from packages will be gathered according to this configuration. */
	UPROPERTY(config, EditAnywhere, Category = "Execution")
	bool IsEnabled;

	/* Packages whose paths match these wildcard patterns, specified relative to the project's root, may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextIncludePath> IncludePathWildcards;

	/* Packages whose paths match these wildcard patterns will be excluded from gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextExcludePath> ExcludePathWildcards;

	/* Packages whose names match these wildcard patterns may be processed for text to gather. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextFileExtension> FileExtensions;

	/* Packages in these collections may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FName> Collections;

	/*
	 * Worlds (including their associated external actors) in these collections may be processed for gathering. The collections may contain more than just worlds, and will be filtered internally.
	 * @note This is typically not needed unless setting up modular localization targets that have to gather external actors separately from their world (eg, when using External Data Layers).
	 */
	UPROPERTY(config, EditAnywhere, Category = "Filter", AdvancedDisplay)
	TArray<FName> WorldCollections;

	/* Classes that should be excluded from gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter", meta=(MetaClass="/Script/CoreUObject.Object", AllowAbstract="True"))
	TArray<FSoftClassPath> ExcludeClasses;

	/* Should classes derived from those in the exclude classes list also be excluded from gathering? */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	bool ShouldExcludeDerivedClasses;

	/* Specifications for how to gather text from specific metadata keys from BP/struct/enum assets. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FMetaDataKeyGatherSpecification> MetadataKeySpecifications;

	/** List of field types (eg, Property, Function, ScriptStruct, Enum, etc) that should be included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldTypesToInclude;

	/** List of field types (eg, Property, Function, ScriptStruct, Enum, etc) the should be excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldTypesToExclude;

	/** List of field owner types (eg, MyClass, MyStruct, etc) that should have fields within them included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldOwnerTypesToInclude;

	/** List of field owner types (eg, MyClass, MyStruct, etc) that should have fields within them excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldOwnerTypesToExclude;

	/** List of outer types (eg, Property, Function, ScriptStruct, etc) of fields that should be included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldOuterTypesToInclude;

	/** List of outer types (eg, Property, Function, ScriptStruct, etc) of fields that should be excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData", AdvancedDisplay)
	TArray<FString> MetadataFieldOuterTypesToExclude;

	/* If enabled, data that is specified as editor-only may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text")
	bool ShouldGatherFromEditorOnlyData;

	/* Should we ignore the cached text in the package header and perform a full package load instead? */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text", AdvancedDisplay)
	bool SkipGatherCache;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FGatherTextFromMetaDataConfiguration
{
	GENERATED_USTRUCT_BODY()

	FGatherTextFromMetaDataConfiguration()
		: IsEnabled(false)
		, ShouldGatherFromEditorOnlyData(false)
	{
	}

	/* If enabled, metadata will be gathered according to this configuration. */
	UPROPERTY(config, EditAnywhere, Category = "Execution")
	bool IsEnabled;

	/* Metadata from source files whose paths match these wildcard patterns, specified relative to the project's root, may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextIncludePath> IncludePathWildcards;

	/* Metadata from source files whose paths match these wildcard patterns will be excluded from gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	TArray<FGatherTextExcludePath> ExcludePathWildcards;

	/* Specifications for how to gather text from specific metadata keys. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FMetaDataKeyGatherSpecification> KeySpecifications;

	/** List of field types (eg, Property, Function, ScriptStruct, Enum, etc) that should be included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldTypesToInclude;

	/** List of field types (eg, Property, Function, ScriptStruct, Enum, etc) the should be excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldTypesToExclude;

	/** List of field owner types (eg, MyClass, MyStruct, etc) that should have fields within them included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldOwnerTypesToInclude;

	/** List of field owner types (eg, MyClass, MyStruct, etc) that should have fields within them excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldOwnerTypesToExclude;

	/** List of outer types (eg, Property, Function, ScriptStruct, etc) of fields that should be included in the gather, or empty to include everything. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldOuterTypesToInclude;

	/** List of outer types (eg, Property, Function, ScriptStruct, etc) of fields that should be excluded from the gather. */
	UPROPERTY(config, EditAnywhere, Category = "MetaData")
	TArray<FString> FieldOuterTypesToExclude;

	/* If enabled, data that is specified as editor-only may be processed for gathering. */
	UPROPERTY(config, EditAnywhere, Category = "Filter")
	bool ShouldGatherFromEditorOnlyData;

	LOCALIZATION_API bool Validate(const bool bIsEngineTarget, FText& OutError) const;
};

USTRUCT()
struct FLocalizationExportingSettings
{
	GENERATED_BODY()

	FLocalizationExportingSettings()
		: CollapseMode(ELocalizedTextCollapseMode::IdenticalTextIdAndSource)
		, POFormat(EPortableObjectFormat::Unreal)
		, ShouldPersistCommentsOnExport(false)
		, ShouldAddSourceLocationsAsComments(true)
	{
	}

	/* How should we collapse down text when exporting to PO? */
	UPROPERTY(config, EditAnywhere, Category = "Collapsing", AdvancedDisplay)
	ELocalizedTextCollapseMode CollapseMode;

	/* Which format of PO file should we use? */
	UPROPERTY(config, EditAnywhere, Category = "Format", AdvancedDisplay)
	EPortableObjectFormat POFormat;

	/* Should user comments in existing PO files be persisted after export? Useful if using a third party service that stores editor/translator notes in the PO format's comment fields. */
	UPROPERTY(config, EditAnywhere, Category = "Comments")
	bool ShouldPersistCommentsOnExport;

	/* Should source locations be added to PO file entries as comments? Useful if a third party service doesn't expose PO file reference comments, which typically store the source location. */
	UPROPERTY(config, EditAnywhere, Category = "Comments")
	bool ShouldAddSourceLocationsAsComments;
};

USTRUCT()
struct FLocalizationCompilationSettings
{
	GENERATED_BODY()

	FLocalizationCompilationSettings()
		: SkipSourceCheck(false)
		, ValidateFormatPatterns(true)
		, ValidateSafeWhitespace(false)
		, ValidateRichTextTags(false)
	{
	}

	/* Should we skip the source check when compiling translations? This will allow translations whose source no longer matches the active source to still be used by the game at runtime. */
	UPROPERTY(config, EditAnywhere, Category="Source")
	bool SkipSourceCheck;

	/* Should we validate that format patterns are valid for the language being compiled (eg, detect invalid plural rules or broken syntax). */
	UPROPERTY(config, EditAnywhere, Category="Validation")
	bool ValidateFormatPatterns;

	/* Should we validate that text doesn't contain any unsafe whitespace (leading or trailing whitespace) that could get lost during the translation process. */
	UPROPERTY(config, EditAnywhere, Category="Validation")
	bool ValidateSafeWhitespace;

	/* Should we validate translated text containing rich text tags to see if the translated text have broken rich text tags. (i.e. "Example of <b>broken tag</") */
	UPROPERTY(config, EditAnywhere, Category = "Validation")
	bool ValidateRichTextTags;
};

USTRUCT()
struct FLocalizationImportDialogueSettings
{
	GENERATED_BODY()

	FLocalizationImportDialogueSettings()
		: ImportedDialogueFolder(TEXT("ImportedDialogue"))
		, bImportNativeAsSource(false)
	{
	}

	/** Path to the folder to import the audio from. This folder is expected to contain language sub-folders, which in turn contain the raw WAV files to import. */
	UPROPERTY(config, EditAnywhere, Category="Dialogue")
	FDirectoryPath RawAudioPath;

	/** Folder in which to create the generated sound waves. This is relative to the root of the L10N language folder (or the root content folder if importing native dialogue as source dialogue). */
	UPROPERTY(config, EditAnywhere, Category="Dialogue")
	FString ImportedDialogueFolder;

	/** Should the dialogue for the native language be imported as if it were source audio? If false, the native language dialogue will be imported as localized data for the native language. */
	UPROPERTY(config, EditAnywhere, Category="Dialogue")
	bool bImportNativeAsSource;
};

USTRUCT()
struct FCultureStatistics
{
	GENERATED_USTRUCT_BODY()

	FCultureStatistics()
		: WordCount(0)
	{
	}

	FCultureStatistics(const FString& InCultureName)
		: CultureName(InCultureName)
		, WordCount(0)
	{
	}


	/* The ISO name for this language. */
	UPROPERTY(config, EditAnywhere, Category = "Language", DisplayName="Language")
	FString CultureName;

	/* The estimated number of words that have been localized for this language. */
	UPROPERTY(Transient, EditAnywhere, Category = "Statistics")
	uint32 WordCount;
};

UENUM()
enum class ELocalizationTargetLoadingPolicy : uint8
{
	/** This target's localization data will never be loaded automatically. */
	Never,
	/** This target's localization data will always be loaded automatically. */
	Always,
	/** This target's localization data will only be loaded when running the editor. Use if this target localizes the editor. */
	Editor,
	/** This target's localization data will only be loaded when running the game. Use if this target localizes your game. */
	Game,
	/** This target's localization data will only be loaded if the editor is displaying localized property names. */
	PropertyNames,
	/** This target's localization data will only be loaded if the editor is displaying localized tool tips. */
	ToolTips,
};

USTRUCT()
struct FLocalizationTargetSettings
{
	GENERATED_USTRUCT_BODY()

	FLocalizationTargetSettings()
		: Guid(FGuid::NewGuid())
		, ConflictStatus(ELocalizationTargetConflictStatus::Unknown)
		, NativeCultureIndex(INDEX_NONE)
	{
	}

	/* Unique name for the target. */
	UPROPERTY(config, EditAnywhere, Category = "Target")
	FString Name;

	UPROPERTY(config, meta=(IgnoreForMemberInitializationTest))
	FGuid Guid;

	/* Whether the target has outstanding conflicts that require resolution. */
	UPROPERTY(Transient, EditAnywhere, Category = "Target")
	ELocalizationTargetConflictStatus ConflictStatus;

	/* Text present in these targets will not be duplicated in this target. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text")
	TArray<FGuid> TargetDependencies;

	/* Text present in these manifests will not be duplicated in this target. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text", AdvancedDisplay, meta=(FilePathFilter="manifest"))
	TArray<FFilePath> AdditionalManifestDependencies;

	/* The names of modules which must be loaded when gathering text. Use to gather from packages or metadata sourced from a module that isn't the primary game module. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text", AdvancedDisplay)
	TArray<FString> RequiredModuleNames;

	/* Parameters for defining what text is gathered from text files and how. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text")
	FGatherTextFromTextFilesConfiguration GatherFromTextFiles;

	/* Parameters for defining what text is gathered from packages and how. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text")
	FGatherTextFromPackagesConfiguration GatherFromPackages;

	/* Parameters for defining what text is gathered from metadata and how. */
	UPROPERTY(config, EditAnywhere, Category = "Gather Text")
	FGatherTextFromMetaDataConfiguration GatherFromMetaData;

	/* Settings for exporting translations. */
	UPROPERTY(config, EditAnywhere, Category = "Export Text", meta=(ShowOnlyInnerProperties))
	FLocalizationExportingSettings ExportSettings;

	/* Settings for compiling translations. */
	UPROPERTY(config, EditAnywhere, Category = "Compile Text", meta=(ShowOnlyInnerProperties))
	FLocalizationCompilationSettings CompileSettings;

	/* Settings for importing dialogue from WAV files. */
	UPROPERTY(config, EditAnywhere, Category = "Import Dialogue", meta=(ShowOnlyInnerProperties))
	FLocalizationImportDialogueSettings ImportDialogueSettings;

	/* The index of the native language among the supported languages. */
	UPROPERTY(config, EditAnywhere, Category = "Languages")
	int32 NativeCultureIndex;

	/* Language for which the source text is being localized for.*/
	UPROPERTY(config, EditAnywhere, Category = "Languages")
	TArray<FCultureStatistics> SupportedCulturesStatistics;
};

UCLASS(MinimalAPI, Within=LocalizationTargetSet)
class ULocalizationTarget : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULocalizationTarget(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(EditAnywhere, Category = "Target", meta=(ShowOnlyInnerProperties))
	FLocalizationTargetSettings Settings;

public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API bool IsMemberOfEngineTargetSet() const;
	UE_API bool UpdateWordCountsFromCSV();
	UE_API void UpdateStatusFromConflictReport();
	UE_API bool RenameTargetAndFiles(const FString& NewName);
	UE_API bool DeleteFiles(const FString* const Culture = nullptr) const;
};

UCLASS(MinimalAPI, Within=LocalizationSettings)
class ULocalizationTargetSet : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Targets")
	TArray<TObjectPtr<ULocalizationTarget>> TargetObjects;

public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
