// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Misc/CString.h"
#include "Templates/SharedPointer.h"

#define UE_API LOCALIZATION_API

class FText;

/**
* Class for handling language codes used in the Portable Object format.
*/
class FPortableObjectCulture
{
public:
	FPortableObjectCulture()
	{}

	UE_API FPortableObjectCulture( const FString& LangCode, const FString& PluralForms = TEXT("") );

	/**
	 * Checks to see if we have a language code and if we were able to match a culture to that code
	 */
	bool IsValid() const { return !LanguageCode.IsEmpty() && Culture.IsValid(); }

	/**
	 * Sets the language code
	 */
	UE_API void SetLanguageCode( const FString& LangCode );

	/**
	 * Retrieves the language code
	 */
	const FString& GetLanguageCode() const { return LanguageCode; }

	/**
	 * Sets the language plural forms.  This is only required if we wish to override the default plural forms associated with a language.
	 */
	void SetPluralForms( const FString& InPluralForms ) { LanguagePluralForms = InPluralForms; }

	/**
	 * Retrieves the language plural forms.
	 */
	UE_API FString GetPluralForms() const;

	/**
	 * Retrieves the two or three letter language code for the culture
	 */
	UE_API FString Language() const;

	/**
	 * Retrieves the country info of the culture
	 */
	UE_API FString Country() const;

	/**
	 * Retrieves the variant info for the culture
	 */
	UE_API FString Variant() const;

	/**
	 * Retrieves the display name for the language.
	 */
	UE_API FString DisplayName() const;

	/**
	 * Retrieves the English name for the language.
	 */
	UE_API FString EnglishName() const;


private:
	/**
	 * Retrieves the language default plural forms.  If no plural forms are provided for this language we use this to pull from a set of defaults if available.
	 */
	UE_API FString GetDefaultPluralForms() const;

	// The language code as it appears in the PO file.
	FString LanguageCode;
	FString LanguagePluralForms;

	FCulturePtr Culture;
};


/**
* Class for representing the key of an entry in a Portable Object file(.po) or a Portable Object Template file(.pot).
*/
class FPortableObjectEntryKey
{
public:
	FPortableObjectEntryKey()
	{
	}

	FPortableObjectEntryKey(const FString& InMsgId, const FString& InMsgIdPlural, const FString& InMsgCtxt)
		: MsgId(InMsgId)
		, MsgIdPlural(InMsgIdPlural)
		, MsgCtxt(InMsgCtxt)
	{
	}

	bool operator==(const FPortableObjectEntryKey& Other) const
	{
		return MsgId.Equals(Other.MsgId, ESearchCase::CaseSensitive) 
			&& MsgIdPlural.Equals(Other.MsgIdPlural, ESearchCase::CaseSensitive) 
			&& MsgCtxt.Equals(Other.MsgCtxt, ESearchCase::CaseSensitive);
	}

	friend inline uint32 GetTypeHash(const FPortableObjectEntryKey& Key)
	{
		uint32 Hash = 0;
		Hash = FLocKey::ProduceHash(Key.MsgId, Hash);
		Hash = FLocKey::ProduceHash(Key.MsgIdPlural, Hash);
		Hash = FLocKey::ProduceHash(Key.MsgCtxt, Hash);
		return Hash;
	}

	/* Represents the original source text(also called the id or context).  Stored here are the msgid values from the Portable Object entries. */
	FString MsgId;

	/* Represents the plural form of the source text.  Stored here are the msgid_plural values from the Portable Object file entries. */
	FString MsgIdPlural;

	/* Represents the disambiguating  context for the source text.  If used, will prevent two identical source strings from getting collapsed into one entry. */
	FString MsgCtxt;
};

/**
* Class for representing entries in a Portable Object file(.po) or a Portable Object Template file(.pot).
*/
class FPortableObjectEntry : public FPortableObjectEntryKey
{
public: 
	FPortableObjectEntry()
	{
	}
	
	/**
	 * Helper function that adds to the extracted comments.
	 *
	 * @param InComment		String representing an extracted comment entry.
	 */
	UE_API void AddExtractedComment( const FString& InComment );

	/**
	 * Helper function that adds to the extracted comments.
	 *
	 * @param InComments	String array representing extracted comment entries.
	 */
	UE_API void AddExtractedComments( const TArray<FString>& InComments );

	/**
	 * Helper function that adds to the translator comments.
	 *
	 * @param InComment		String representing a translator comment entry.
	 */
	UE_API void AddTranslatorComment(const FString& InComment);

	/**
	 * Helper function that adds to the translator comments.
	 *
	 * @param InComments	String array representing the translator's notes.
	 */
	UE_API void AddTranslatorComments(const TArray<FString>& InComments);

	/**
	 * Helper function that adds to the reference comments.
	 *
	 * @param InReference	String representing reference comment entries of the following form: "File/path/file.cpp:20 File/path/file.cpp:21 File/path/file2.cpp:5".
	 */
	UE_API void AddReference( const FString& InReference );

	/**
	 * Helper function that adds to the reference comments.
	 *
	 * @param InReferences	String array representing reference comment entries of the following form: "File/path/file.cpp:20 File/path/file.cpp:21 File/path/file2.cpp:5".
	 */
	UE_API void AddReferences( const TArray<FString>& InReferences );

	/**
	 * Function to convert the entry to a string.
	 *
	 * @return	String representing the entry that can be written directly to the .po file.
	 */
	UE_API FString ToString() const;

public:
	/* Represents the translated text.  This stores the msgstr, msgstr[0], msgstr[1], etc values from Portable Object entries. */
	TArray< FString > MsgStr;

	/* Stores extracted comments. Lines starting with #. above the msgid.  
	
		#. TRANSLATORS: A test phrase with all letters of the English alphabet.
		#. Replace it with a sample text in your language, such that it is
		#. representative of language's writing system.
		msgid "The Quick Brown Fox Jumps Over The Lazy Dog"
		msgstr ""
	*/
	TArray<FString> ExtractedComments;

	/* Stores the translator comments.  Lines starting with # (hash and space), followed by any text whatsoever.
	
		# Wikipedia says that 'etrurski' is our name for this script.
		msgid "Old Italic"
		msgstr "etrurski"
	*/
	TArray<FString> TranslatorComments;

	/* Stores a reference comments. Lines starting with #: above the msgid.  
	    
		#: /Engine/Source/Runtime/Engine/Private/Actor.cpp:2306
		#: /Engine/Source/Runtime/Engine/Private/Actor.cpp:2307 /Engine/Source/Runtime/Engine/Private/Actor.cpp:2308
		msgid "The Quick Brown Fox Jumps Over The Lazy Dog"
		msgstr ""
	*/
	TArray<FString> ReferenceComments;

	/* Stores flags.  Lines starting with #,  
	    
		#, fuzzy
		msgid "The Quick Brown Fox Jumps Over The Lazy Dog"
		msgstr ""
	*/
	TArray<FString> Flags;

	/* Stores any unknown elements we may encounter when processing a Portable Object file.  */
	TArray< FString > UnknownElements;
};

typedef TMap<FPortableObjectEntryKey, TSharedPtr<FPortableObjectEntry>> FPortableObjectEntries;

/**
* Class that stores and manipulates PO and POT file header info.
*/
class FPortableObjectHeader
{
public:
	typedef TPair<FString, FString> FPOHeaderEntry;
	typedef TArray< FPOHeaderEntry > FPOHeaderData;

	FPortableObjectHeader()
	{
	}

	/**
	 * Creates a string representation of the Portable Object header.
	 *
	 * @return	String representing the header that can be written directly to the .po file.
	 */
	UE_API FString ToString() const;

	/**
	 * Parses out header key/value pair entries.
	 *
	 * @param InStr	String representing a .po or .pot file header
	 * @return	Returns true if successful, false otherwise.
	 */
	bool FromString( const FString& InStr ) {return false;}

	/**
	 * Parses out header info from a FPortableObjectEntry with an empty ID.
	 *
	 * @param LocEntry	The Portable Object entry with an empty ID
	 * @return	Returns true if successful, false otherwise.
	 */
	UE_API bool FromLocPOEntry( const TSharedRef<const FPortableObjectEntry> LocEntry );
	
	/** Checks if a header entry exists with the given key */
	UE_API bool HasEntry ( const FString& EntryKey ) const;

	/** Gets the header entry value for the given key or the empty string if the key does not exist */
	UE_API FString GetEntryValue ( const FString& EntryKey ) const;

	/** Sets a header entry value */
	UE_API void SetEntryValue( const FString& EntryKey, const FString& EntryValue );
	
	/**
	 * Puts the current time into POT-Creation-Date and PO-Revision-Date entries
	 */
	UE_API void UpdateTimeStamp();

	/**
	 * Clears the header entries.
	 */
	void Clear() { HeaderEntries.Empty(); Comments.Empty(); }

private:
	UE_API FPortableObjectHeader::FPOHeaderEntry* FindEntry( const FString& EntryKey );
	UE_API const FPortableObjectHeader::FPOHeaderEntry* FindEntry( const FString& EntryKey ) const;

public:

	/* Stores the header comment block */
	TArray<FString> Comments;
	/* Stores all the header key/value pairs*/
	FPOHeaderData HeaderEntries;
};

/**
* Contains all the info we need to represent files in the Portable Object format.
*/
class FPortableObjectFormatDOM
{
public:
	FPortableObjectFormatDOM()
	{
	}

	FPortableObjectFormatDOM( const FString& LanguageCode )
		: Language( LanguageCode )
	{
	}

	/** Copying is not supported */
	FPortableObjectFormatDOM(const FPortableObjectFormatDOM&) = delete;
	FPortableObjectFormatDOM& operator=(const FPortableObjectFormatDOM&) = delete;

	/**
	 * Creates a string representation of the Portable Object.
	 *
	 * @return	String representing the Portable Object that can be written directly to a file.
	 */
	UE_API FString ToString();
	UE_API void ToString(FString& OutString);

	/**
	 * Parses Portable Object elements from a string.
	 *
	 * @param InStr	String representing a Portable Object file(.PO) or Portable Object Template file(.POT).
	 * @param OutErrorMsg Optional Text to be filled with error information.
	 * @return	Returns true if successful, false otherwise.
	 */
	UE_API bool FromString( const FString& InStr, FText* OutErrorMsg = nullptr );

	/** Creates a header entry based on the project and language info. */
	UE_API void CreateNewHeader( const FString& InCopyrightNotice = FString() );

	/**
	 * Sets a header entry value.
	 * @note Must be called after CreateNewHeader.
	 */
	UE_API void SetHeaderValue(const FString& EntryKey, const FString& EntryValue);

	/**
	 * Sets the language.
	 *
	 * @param LanguageCode	String representing a Portable Object language code.
	 * @param LangPluralForms	Optional plural forms for the language.  If not provided, defaults will be used if found.
	 * @return	Returns true if the function was able to successfully set the language code and pair it with a known culture, false otherwise.
	 */
	UE_API bool SetLanguage( const FString& LanguageCode, const FString& LangPluralForms = TEXT("") );

	/**
	 * Adds a translation entry to the Portable Object.
	 *
	 * @param LocEntry	The LocPOEntry to add.
	 * @return	Returns true if successful, false otherwise.
	 */
	UE_API bool AddEntry( const TSharedRef<FPortableObjectEntry> LocEntry );

	/* Returns the Portable Object entry that matches the passed in entry or NULL if not found */
	UE_API TSharedPtr<FPortableObjectEntry> FindEntry( const TSharedRef<const FPortableObjectEntry> LocEntry ) const;

	/* Returns the Portable Object file entry that matches the passed in parameters or NULL if not found */
	UE_API TSharedPtr<FPortableObjectEntry> FindEntry( const FString& MsgId, const FString& MsgIdPlural, const FString& MsgCtxt ) const;

	/* Sets the project name that will appear in the Project-Id-Version header entry */
	void SetProjectName( const FString& ProjName ){ ProjectName = ProjName; }

	/* Sets the project name from the Project-Id-Version header entry */
	FString GetProjectName() const { return ProjectName; }

	FPortableObjectEntries::TConstIterator GetEntriesIterator() const
	{
		return Entries.CreateConstIterator();
	}

	UE_API void SortEntries();

private:
	FPortableObjectCulture Language;
	FPortableObjectHeader Header;
	FString ProjectName;
	FPortableObjectEntries Entries;
};

#undef UE_API
