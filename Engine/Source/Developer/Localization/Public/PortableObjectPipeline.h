// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "PortableObjectPipeline.generated.h"

class FLocMetadataObject;
class FLocTextHelper;
class FPortableObjectEntry;
class FPortableObjectFormatDOM;

UENUM()
enum class ELocalizedTextCollapseMode : uint8
{
	/** Collapse texts with the same text identity (namespace + key) and source text (default 4.15+ behavior). */
	IdenticalTextIdAndSource			UMETA(DisplayName = "Identical Text Identity (Namespace + Key) and Source Text"),
	/** Collapse texts with the same package ID, text identity (namespace + key), and source text (deprecated 4.14 behavior, removed in 4.17). */
	IdenticalPackageIdTextIdAndSource	UMETA(DisplayName = "Identical Package ID, Text Identity (Namespace + Key) and Source Text", Hidden),
	/** Collapse texts with the same namespace and source text (legacy pre-4.14 behavior). */
	IdenticalNamespaceAndSource			UMETA(DisplayName = "Identical Namespace and Source Text"),
};

UENUM()
enum class EPortableObjectFormat : uint8
{
	/**
	 * The PO file uses the Unreal format.
	 *
	 * When using the "Identical Text Identity and Source Text" collapse mode:
	 *	- msgctxt contains the Unreal identity of the entry.
	 *	- msgid contains the source string.
	 *	- msgstr contains the translation.
	 *
	 * When using the "Identical Namespace and Source Text" collapse mode:
	 *	- msgctxt contains the Unreal namespace of the entry.
	 *	- msgid contains the source string.
	 *	- msgstr contains the translation.
	 */
	Unreal,

	/**
	 * The PO file uses the Crowdin format.
	 *
	 * When using the "Identical Text Identity and Source Text" collapse mode:
	 *	- msgctxt is unused.
	 *	- msgid contains the Unreal identity of the entry.
	 *	- msgstr contains the source string (for the native culture), or the translation (for foreign cultures).
	 *	- X-Crowdin-SourceKey header attribute specifies that msgstr is used as the source text from the native culture.
	 *
	 * When using the "Identical Namespace and Source Text" collapse mode:
	 *	- msgctxt contains the Unreal namespace of the entry.
	 *	- msgid contains the source string.
	 *	- msgstr contains the translation.
	 */
	Crowdin,
};

namespace PortableObjectPipeline
{
	/** Update the given LocTextHelper with the translation data imported from the PO file for the given culture */
	LOCALIZATION_API bool Import(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat);

	/** Update the given LocTextHelper with the translation data imported from the PO file for all cultures */
	LOCALIZATION_API bool ImportAll(FLocTextHelper& InLocTextHelper, const FString& InPOCultureRootPath, const FString& InPOFilename, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bUseCultureDirectory);

	/** Use the given LocTextHelper to generate a new PO file using the translation data for the given culture */
	LOCALIZATION_API bool Export(FLocTextHelper& InLocTextHelper, const FString& InCulture, const FString& InPOFilePath, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bShouldPersistComments);

	/** Use the given LocTextHelper to generate a new PO file using the translation data for all cultures */
	LOCALIZATION_API bool ExportAll(FLocTextHelper& InLocTextHelper, const FString& InPOCultureRootPath, const FString& InPOFilename, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat, const bool bShouldPersistComments, const bool bUseCultureDirectory);

	/** Given a namespace and key, condition this information so it can be written to the "msgctxt" or "msgid" field of a PO file */
	LOCALIZATION_API FString ConditionIdentityForPO(const FString& Namespace, const FString& Key, const TSharedPtr<FLocMetadataObject>& KeyMetaData, const ELocalizedTextCollapseMode InTextCollapseMode);

	/** Given the "msgctxt" or "msgid" field of a PO file, split it into the namespace and key */
	LOCALIZATION_API void ParseIdentityFromPO(const FString& InIdentity, FString& OutNamespace, FString& OutKey);

	/** Given a string, condition it so it can be written as a field of a PO file */
	LOCALIZATION_API FString ConditionArchiveStrForPO(const FString& InStr);

	/** Given the field of a PO file, condition it back to a clean string */
	LOCALIZATION_API FString ConditionPOStringForArchive(const FString& InStr);

	/** Given a key string, condition it so it can be written as the extracted comment field of a PO file */
	LOCALIZATION_API FString GetConditionedKeyForExtractedComment(const FString& Key);

	/** Given a source location string, condition it so it can be written as the extracted comment field of a PO file */
	LOCALIZATION_API FString GetConditionedReferenceForExtractedComment(const FString& PORefString);

	/** Given developer-entered translator context notes, condition it so it can be written as the extracted comment field of a PO file */
	LOCALIZATION_API FString GetConditionedDevNotesForExtractedComment(const FString& DevNotes);

	/** Given a meta-data value string, condition it so it can be written as the extracted comment field of a PO file */
	LOCALIZATION_API FString GetConditionedInfoMetaDataForExtractedComment(const FString& KeyName, const FString& ValueString);

	/** Given the collapse mode and PO format, append any extra required meta-data to the PO file header */
	LOCALIZATION_API void UpdatePOFileHeaderForSettings(FPortableObjectFormatDOM& PortableObject, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat);

	/** Populate the basic data (msgctxt, msgid, msgstr) within a PO entry */
	LOCALIZATION_API void PopulateBasicPOFileEntry(FPortableObjectEntry& POEntry, const FString& InNamespace, const FString& InKey, const TSharedPtr<FLocMetadataObject>& InKeyMetaData, const FString& InSourceString, const FString& InTranslation, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat);

	/** Parse the basic data (msgctxt, msgid, msgstr) from a PO entry */
	LOCALIZATION_API void ParseBasicPOFileEntry(const FPortableObjectEntry& POEntry, FString& OutNamespace, FString& OutKey, FString& OutSourceString, FString& OutTranslation, const ELocalizedTextCollapseMode InTextCollapseMode, const EPortableObjectFormat InPOFormat);
}
