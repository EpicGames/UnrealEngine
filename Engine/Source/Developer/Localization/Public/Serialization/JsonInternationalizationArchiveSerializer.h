// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/LocKeyFuncs.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Templates/SharedPointer.h"

#define UE_API LOCALIZATION_API

class FArchiveEntry;
class FInternationalizationArchive;
class FInternationalizationManifest;

/**
 * Implements a serializer that serializes to and from Json encoded data.
 */
class FJsonInternationalizationArchiveSerializer
{
public:

	/**
	 * Deserializes an archive from a JSON string.
	 *
	 * @param InStr				The JSON string to deserialize from.
	 * @param InArchive			The archive to populate from the JSON data.
	 * @param InManifest		The manifest associated with the archive. May be null, but you won't be able to load archives with a version < FInternationalizationArchive::EFormatVersion::AddedKeys.
	 * @param InNativeArchive	The native archive associated with the archive. May be null.
	 *
	 * @return true if deserialization was successful, false otherwise.
	 */
	static UE_API bool DeserializeArchiveFromString(const FString& InStr, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive);

	/**
	 * Deserializes an archive from a JSON file.
	 *
	 * @param InJsonFile		The path to the JSON file to deserialize from.
	 * @param InArchive			The archive to populate from the JSON data.
	 * @param InManifest		The manifest associated with the archive. May be null, but you won't be able to load archives with a version < FInternationalizationArchive::EFormatVersion::AddedKeys.
	 * @param InNativeArchive	The native archive associated with the archive. May be null.
	 *
	 * @return true if deserialization was successful, false otherwise.
	 */
	static UE_API bool DeserializeArchiveFromFile(const FString& InJsonFile, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive);

	/**
	 * Serializes an archive to a JSON string.
	 *
	 * @param InArchive			The archive data to serialize.
	 * @param Str				The string to fill with the JSON data.
	 *
	 * @return true if serialization was successful, false otherwise.
	 */
	static UE_API bool SerializeArchiveToString(TSharedRef<const FInternationalizationArchive> InArchive, FString& Str);

	/**
	* Serializes an archive to a JSON string.
	*
	* @param InArchive			The archive data to serialize.
	* @param InJsonFile			The path to the JSON file to serialize to.
	*
	* @return true if serialization was successful, false otherwise.
	*/
	static UE_API bool SerializeArchiveToFile(TSharedRef<const FInternationalizationArchive> InArchive, const FString& InJsonFile);

private:

	/**
	 * Used to arrange Internationalization archive data in a hierarchy based on namespace prior to json serialization.
	 */
	struct FStructuredArchiveEntry
	{
		explicit FStructuredArchiveEntry(const FString& InNamespace)
			: Namespace(InNamespace)
		{
		}

		const FString Namespace;
		TMap<FString, TSharedPtr<FStructuredArchiveEntry>, FDefaultSetAllocator, FLocKeyMapFuncs<TSharedPtr<FStructuredArchiveEntry>>> SubNamespaces;
		TArray<TSharedPtr<FArchiveEntry>> ArchiveEntries;
	};

	/**
	 * Deserializes an archive from a JSON object.
	 *
	 * @param InJsonObj			The JSON object to serialize from.
	 * @param InArchive			The archive to populate from the JSON data.
	 * @param InManifest		The manifest associated with the archive. May be null, but you won't be able to load archives with a version < FInternationalizationArchive::EFormatVersion::AddedKeys.
	 * @param InNativeArchive	The native archive associated with the archive. May be null.
	 *
	 * @return true if deserialization was successful, false otherwise.
	 */
	static bool DeserializeInternal(UE::Json::FConstObject InJsonObj, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive);

	/**
	 * Convert a Internationalization archive to a JSON object.
	 *
	 * @param InArchive The Internationalization archive object to serialize from.
	 * @param JsonObj The Json object that will store the data.
	 * @return true if serialization was successful, false otherwise.
	 */
	static bool SerializeInternal(TSharedRef<const FInternationalizationArchive> InArchive, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator);

	/**
	 * Recursive function that will traverse the JSON object and populate an archive.
	 *
	 * @param InJsonObj			The JSON object to serialize from.
	 * @param ParentNamespace	The namespace of the parent JSON object.
	 * @param InArchive			The archive to populate from the JSON data.
	 * @param InManifest		The manifest associated with the archive. May be null, but you won't be able to load archives with a version < FInternationalizationArchive::EFormatVersion::AddedKeys.
	 * @param InNativeArchive	The native archive associated with the archive. May be null.
	 *
	 * @return true if successful, false otherwise.
	 */
	static bool JsonObjToArchive(UE::Json::FConstObject InJsonObj, const FString& ParentNamespace, TSharedRef<FInternationalizationArchive> InArchive, TSharedPtr<const FInternationalizationManifest> InManifest, TSharedPtr<const FInternationalizationArchive> InNativeArchive);

	/**
	 * Takes a Internationalization archive and arranges the data into a hierarchy based on namespace.
	 *
	 * @param InArchive The Internationalization archive.
	 * @param RootElement The root element of the structured data.
	 */
	static void GenerateStructuredData( TSharedRef< const FInternationalizationArchive > InArchive, TSharedPtr< FStructuredArchiveEntry > RootElement );

	/**
	 * Goes through the structured, hierarchy based, archive data and does a non-culture specific sort on namespaces and default text.
	 *
	 * @param RootElement The root element of the structured data.
	 */
	static void SortStructuredData( TSharedPtr< FStructuredArchiveEntry > InElement );

	/**
	 * Populates a JSON object from Internationalization archive data that has been structured based on namespace.
	 *
	 * @param InElement Internationalization archive data structured based on namespace.
	 * @param JsonObj JSON object to be populated.
	 */
	static void StructuredDataToJsonObj(TSharedPtr< const FStructuredArchiveEntry > InElement, UE::Json::FObject JsonObj, UE::Json::FAllocator& Allocator);

	static const FString TAG_FORMATVERSION;
	static const FString TAG_NAMESPACE;
	static const FString TAG_KEY;
	static const FString TAG_CHILDREN;
	static const FString TAG_SUBNAMESPACES;
	static const FString TAG_DEPRECATED_DEFAULTTEXT;
	static const FString TAG_DEPRECATED_TRANSLATEDTEXT;
	static const FString TAG_OPTIONAL;
	static const FString TAG_SOURCE;
	static const FString TAG_SOURCE_TEXT;
	static const FString TAG_TRANSLATION;
	static const FString TAG_TRANSLATION_TEXT;
	static const FString TAG_METADATA;
	static const FString TAG_METADATA_KEY;
	static const FString NAMESPACE_DELIMITER;
};

#undef UE_API
