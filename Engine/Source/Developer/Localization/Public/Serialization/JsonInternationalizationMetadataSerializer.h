// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Templates/SharedPointer.h"

#define UE_API LOCALIZATION_API

class FLocMetadataObject;

class FJsonInternationalizationMetaDataSerializer
{
public:
	/**
	 * Deserializes manifest metadata from a JSON object.
	 *
	 * @param JsonObj The JSON object to deserialize from.
	 * @param OutMetadataObj The resulting metadata object, or an unmodified object if the serialize is not successful.
	 */
	static UE_API void DeserializeMetadata( UE::Json::FConstObject JsonObj, TSharedPtr< FLocMetadataObject >& OutMetadataObj );

	/**
	 * Serializes manifest metadata object to a JSON object.
	 *
	 * @param Metadata The Internationalization metadata to serialize.
	 * @param OutJsonObj The resulting JSON object or an unmodified object if the serialize is not successful.
	 */
	static UE_API void SerializeMetadata( const TSharedRef< FLocMetadataObject > Metadata, UE::Json::FValue& OutJsonObj, UE::Json::FAllocator& Allocator );

	/**
	 * Utility function that will convert metadata to string using the JSON metadata serializer.
	 *
	 * @param Metadata The Internationalization metadata to convert to string.
	 */
	static UE_API FString MetadataToString( const TSharedPtr<FLocMetadataObject> Metadata );
};

#undef UE_API
