// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/TextKey.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"

class FLocTextHelper;
class FName;
class FTextLocalizationMetaDataResource;
class FTextLocalizationResource;

enum class EGenerateLocResFlags : uint8
{
	/** No special behavior */
	None = 0,
	/** Should "stale" translations (translations that were made against different source text) still be used? */
	AllowStaleTranslations = 1<<0,
	/** Validate that format patterns are valid for the culture being compiled (eg, detect invalid plural rules or broken syntax) */
	ValidateFormatPatterns = 1<<1,
	/** Validate that text doesn't contain any unsafe whitespace (leading or trailing whitespace) that could get lost during the translation process */
	ValidateSafeWhitespace = 1<<2,
	/** Validate that rich text tags have not been broken during translation */
	ValidateRichTextTags = 1<<3,
};
ENUM_CLASS_FLAGS(EGenerateLocResFlags);

/** Utility functions for generating compiled LocMeta (Localization MetaData Resource) and LocRes (Localization Resource) files from source localization data */
class FTextLocalizationResourceGenerator
{
public:
	/**
	 * Given a loc text helper, generate a compiled LocMeta resource.
	 */
	LOCALIZATION_API static bool GenerateLocMeta(const FLocTextHelper& InLocTextHelper, const FString& InResourceName, FTextLocalizationMetaDataResource& OutLocMeta);

	/**
	 * Given a loc text helper, generate a compiled LocRes resource for the given culture.
	 */
	LOCALIZATION_API static bool GenerateLocRes(const FLocTextHelper& InLocTextHelper, const FString& InCultureToGenerate, const EGenerateLocResFlags InGenerateFlags, const FTextKey& InLocResID, FTextLocalizationResource& OutPlatformAgnosticLocRes, TMap<FName, TSharedRef<FTextLocalizationResource>>& OutPerPlatformLocRes, const int32 InPriority = 0);

	/**
	 * Given a config file, generate a compiled LocRes resource for the active culture and use it to update the live-entries in the localization manager.
	 */
	LOCALIZATION_API static bool GenerateLocResAndUpdateLiveEntriesFromConfig(const FString& InConfigFilePath, const EGenerateLocResFlags InGenerateFlags);
};
