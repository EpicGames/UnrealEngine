// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#include "PortableObjectPipeline.h"
#include "Internationalization/LocalizedTextSourceTypes.h"

#include "UserGeneratedContentLocalization.generated.h"

#define UE_API LOCALIZATION_API

class IPlugin;
class FJsonObject;
class FLocFileSCCNotifies;
class FLocTextHelper;

/**
 * Settings controlling UGC localization.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UUserGeneratedContentLocalizationSettings : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * List of cultures that should be disabled for UGC localization.
	 * @note You can't disable the native culture for the project.
	 */
	UPROPERTY(config, EditAnywhere, Category=Localization)
	TArray<FString> CulturesToDisable;

	/**
	 * Should we compile UGC localization (if present) for DLC plugins during cook?
	 */
	UPROPERTY(config, EditAnywhere, Category=Localization)
	bool bCompileDLCLocalizationDuringCook = true;

	/**
	 * Should we validate UGC localization (if present) for DLC plugins during cook?
	 * @note Validation will happen against a UGC localization descriptor that has had InitializeFromProject called on it.
	 */
	UPROPERTY(config, EditAnywhere, Category=Localization)
	bool bValidateDLCLocalizationDuringCook = true;
};

/**
 * Minimal descriptor needed to generate a localization target for UGC localization.
 */
USTRUCT()
struct FUserGeneratedContentLocalizationDescriptor
{
	GENERATED_BODY()
	
public:
	/**
	 * Initialize the NativeCulture and CulturesToGenerate values based on the settings of the currently loaded Unreal project.
	 * @param LocalizationCategory What category is the localization targets being used with this descriptor?
	 */
	UE_API void InitializeFromProject(const ELocalizedTextSourceCategory LocalizationCategory = ELocalizedTextSourceCategory::Game);
	
	/**
	 * Validate that this descriptor isn't using cultures that aren't present in the CulturesToGenerate of the given default.
	 *   - If the NativeCulture is invalid, reset it to the value from the default.
	 *   - If CulturesToGenerate contains invalid entries then remove those from the array.
	 * 
	 * @return True if this descriptor was valid and no changes were made. False if this descriptor was invalid and had default changes applied.
	 */
	UE_API bool Validate(const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor);

	/**
	 * Save the settings to a JSON object/file.
	 */
	UE_API bool ToJsonObject(TSharedPtr<FJsonObject>& OutJsonObject) const;
	UE_API bool ToJsonString(FString& OutJsonString) const;
	UE_API bool ToJsonFile(const TCHAR* InFilename) const;

	/**
	 * Load the settings from a JSON object/file.
	 */
	UE_API bool FromJsonObject(TSharedRef<const FJsonObject> InJsonObject);
	UE_API bool FromJsonString(const FString& InJsonString);
	UE_API bool FromJsonFile(const TCHAR* InFilename);
	
	/**
	 * The language that the source text is authored in.
	 * @note You shouldn't change this once you start to localize your text.
	 */
	UPROPERTY(EditAnywhere, Category=Localization, DisplayName="Native Language")
	FString NativeCulture;

	/**
	 * The languages that we should generate localization data for.
	 * @note Will implicitly always contain the native language during export/compile.
	 */
	UPROPERTY(EditAnywhere, Category=Localization, DisplayName="Languages to Generate")
	TArray<FString> CulturesToGenerate;

	/**
	 * What format of PO file should we use?
	 * @note You can adjust this later and we'll attempt to preserve any existing localization data by importing with the old setting prior to export.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Localization, DisplayName="PO Format")
	EPortableObjectFormat PoFormat = EPortableObjectFormat::Unreal;
};

/**
 * UGC localization can be used to provide a simplified localization experience for basic plugins (only providing the PO files to be translated), 
 * and is primarily designed for DLC plugins where the UGC localization will be compiled during cook (@see UUserGeneratedContentLocalizationSettings).
 * 
 * Support for non-DLC plugins can be provided via project specific tooling built upon this base API.
 * Support for complex plugins (such as those containing different kinds of modules, eg) a mix of game/engine and editor) are not supported via this API.
 */
namespace UserGeneratedContentLocalization
{

enum class EMergeLocalizationMode : uint8;

struct FExportLocalizationOptions
{
	/** Common export options for all plugins */
	FUserGeneratedContentLocalizationDescriptor UGCLocDescriptor;

	/** Optional mapping of plugin names to collection names (to act as a filter for their asset gather step) */
	TMap<FString, FString> PluginNameToCollectionNameFilter;

	/** True to gather localization from source code (if a plugin has a Config or Source folder) */
	bool bGatherSource = true;

	/** True to gather localization from assets */
	bool bGatherAssets = true;

	/** True to gather localization from Verse */
	bool bGatherVerse = true;

	/** True to update the plugin descriptors (if needed) so that they contain the exported localization target */
	bool bUpdatePluginDescriptor = true;

	/** True to automatically clean-up any scratch data created during the localization export */
	bool bAutoCleanup = true;

	/** The category to use for the exported localization target (only used when bUpdatePluginDescriptor is true) */
	ELocalizedTextSourceCategory LocalizationCategory = ELocalizedTextSourceCategory::Game;

	/** An optional copyright notice to insert into the exported files */
	FString CopyrightNotice;

	/** An optional override for the root directory that we import/export the localization target data to, generating a sub-folder for each target exported (@see GetLocalizationTargetName, @see GetLocalizationTargetDirectory) */
	FString LocalizationTargetRootDirectoryOverride;

	/** If set, merge any existing plugin localization data with the data copied from LocalizationTargetRootDirectoryOverride (in the export scratch directory) prior to running the export */
	TOptional<EMergeLocalizationMode> MergeProjectDataWithRootDirectoryOverrideData;
};

enum class ELoadLocalizationResult : uint8
{
	/** There was no source localization data to load */
	NoData,
	/** There was source localization data to load, but we failed to load it */
	Failed,
	/** There was source localization data to load, and we successfully loaded it */
	Success,
};

enum class EMergeLocalizationMode : uint8
{
	/**
	 * Only merge untranslated strings
	 */
	Untranslated,

	/**
	 * Merge all strings, even if they have an existing translation
	 * @note This will overwrite any existing translation data
	 */
	All,
};

/**
 * Gets the name of the custom plugin field used to store the localization target name.
 */
LOCALIZATION_API FStringView GetLocalizationTargetNameFieldName();

/**
 * Utility to get the name of the UGC localization target that would be used by the given plugin.
 */
LOCALIZATION_API FString GetLocalizationTargetName(const TSharedRef<IPlugin>& Plugin);

#if WITH_EDITOR

/**
 * Utility to set the name of the UGC localization target that will be used by the given plugin.
 */
LOCALIZATION_API bool SetLocalizationTargetName(const TSharedRef<IPlugin>& Plugin, const FString& LocalizationTargetName, bool bUseSourceControl, FText& OutFailReason);

#endif // WITH_EDITOR

/**
 * Utility to get the path of the UGC localization target directory that would be used by the given plugin.
 */
LOCALIZATION_API FString GetLocalizationTargetDirectory(const TSharedRef<IPlugin>& Plugin);

/**
 * Utility to get the path of the UGC localization target directory that would be used by the given plugin, based on the given root directory override (@see FExportLocalizationOptions).
 */
LOCALIZATION_API FString GetLocalizationTargetDirectory(const TSharedRef<IPlugin>& Plugin, const FString& LocalizationTargetRootDirectoryOverride);

/**
 * Utility to get the path of the UGC localization target that would be used by the given target name and directory.
 */
LOCALIZATION_API FString GetLocalizationTargetDirectory(const FString& LocalizationTargetName, const FString& PluginContentDirectory);

/**
 * Utility to get the path of the UGC localization target that would be used by the given target name and directory, based on the given root directory override (@see FExportLocalizationOptions).
 */
LOCALIZATION_API FString GetLocalizationTargetDirectory(const FString& LocalizationTargetName, const FString& PluginContentDirectory, const FString& LocalizationTargetRootDirectoryOverride);

/**
 * Utility to get the UGCLoc file for for the given UGC localization target name and directory.
 */
LOCALIZATION_API FString GetLocalizationTargetUGCLocFile(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory);

/**
 * Utility to get the PO file for for the given UGC localization target name and directory, and the given culture.
 */
LOCALIZATION_API FString GetLocalizationTargetPOFile(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory, const FString& Culture);

/**
 * Utility to prepare a SCC managed file for writing.
 */
UE_DEPRECATED(5.8, "PreWriteFileWithSCC is deprecated in favor of CreateLocFileNotifies, as the latter can batch SCC requests via BeginParallelTasks/EndParallelTasks")
LOCALIZATION_API void PreWriteFileWithSCC(const FString& Filename, const bool bUseSourceControl = true);

/**
 * Utility to update a SCC managed file after writing.
 */
UE_DEPRECATED(5.8, "PostWriteFileWithSCC is deprecated in favor of CreateLocFileNotifies, as the latter can batch SCC requests via BeginParallelTasks/EndParallelTasks")
LOCALIZATION_API void PostWriteFileWithSCC(const FString& Filename, const bool bUseSourceControl = true);

/**
 * Create a loc file notifies helper that will use source control if currently enabled.
 */
LOCALIZATION_API FLocFileSCCNotifies CreateLocFileNotifies(const bool bUseSourceControl = true);

/**
 * Export UGC localization for the given plugins.
 * 
 * @param Plugins				The list of plugins to export.
 * @param ExportOptions			Options controlling how to export the localization data.
 * @param CommandletExecutor	Callback used to actually execute the gather commandlet:
 *									This should execute an editor with `-run=GatherText -config="..."`, where the config argument is the first argument passed to this callback.
 *									The second argument should be filled with the raw log output from running the commandlet process.
 *									The return value is the exit code of the commandlet process (where zero means success).
 */
LOCALIZATION_API bool ExportLocalization(TArrayView<const TSharedRef<IPlugin>> Plugins, const FExportLocalizationOptions& ExportOptions, TFunctionRef<int32(const FString&, FString&)> CommandletExecutor);

/**
 * Compile UGC localization (if present) for the given plugins, producing LocMeta and LocRes files for consumption by the engine.
 * 
 * @param Plugins				The list of plugins to compile.
 * @param DefaultDescriptor		An optional default UGC localization descriptor to validate any loaded UGC localization descriptors against prior to compiling the localization data.
 */
LOCALIZATION_API bool CompileLocalization(TArrayView<const TSharedRef<IPlugin>> Plugins, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor = nullptr);

/**
 * Compile UGC localization (if present) for the given localization target, producing LocMeta and LocRes files for consumption by the engine.
 *
 * @param LocalizationTargetName			The target name being compiled (@see GetLocalizationTargetName).
 * @param LocalizationTargetInputDirectory	The directory we'll read the source localization data from when compiling the plugin (@see GetLocalizationTargetDirectory).
 * @param LocalizationTargetOutputDirectory	The directory we'll write the LocMeta and LocRes data to when compiling the plugin.
 * @param DefaultDescriptor					An optional default UGC localization descriptor to validate any loaded UGC localization descriptors against prior to compiling the localization data.
 */
LOCALIZATION_API bool CompileLocalization(const FString& LocalizationTargetName, const FString& LocalizationTargetInputDirectory, const FString& LocalizationTargetOutputDirectory, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor = nullptr);

/**
 * Load UGC localization source data for the given localization target.
 * @note This is typically only needed for compilation (which does it internally), but can also be useful if you have other processes that need to read the source data.
 *
 * @param LocalizationTargetName		The target name being loaded (@see GetLocalizationTargetName).
 * @param LocalizationTargetDirectory	The directory we'll read the source localization data from (@see GetLocalizationTargetDirectory).
 * @param OutLocTextHelper				The LocTextHelper to fill with manifest/archive data, re-generated from the source data.
 * @param DefaultDescriptor				An optional default UGC localization descriptor to validate any loaded UGC localization descriptors against prior to loading the localization data.
 */
LOCALIZATION_API ELoadLocalizationResult LoadLocalization(const FString& LocalizationTargetName, const FString& LocalizationTargetDirectory, TSharedPtr<FLocTextHelper>& OutLocTextHelper, const FUserGeneratedContentLocalizationDescriptor* DefaultDescriptor = nullptr);

/**
 * Merge the PO file data of two UGC localization targets together, so that any PO entries with translations from source are present in dest. Whether the translation from source is actually applied to dest depends on the MergeMode.
 * @note Dest does not have to currently exist prior to the merge.
 * 
 * @param SourceLocalizationTargetName		The name of the source localization target (@see GetLocalizationTargetName).
 * @param SourceLocalizationTargetDirectory	The directory of the source localization target (@see GetLocalizationTargetDirectory).
 * @param DestLocalizationTargetName		The name of the dest localization target (@see GetLocalizationTargetName).
 * @param DestLocalizationTargetDirectory	The directory of the dest localization target (@see GetLocalizationTargetDirectory).
 * @param MergeMode							When should an existing translation from source be applied to dest?
 * @param bOnlyIfSourceFileNewer			True if we should only merge from source -> dest if the source file is newer than the dest file.
 * @param bUseSourceControl					Pass false as an optimization if you knew for certain that DestLocalizationTargetDirectory is outside of source control, otherwise pass true.
 */
LOCALIZATION_API bool MergeLocalization(const FString& SourceLocalizationTargetName, const FString& SourceLocalizationTargetDirectory, const FString& DestLocalizationTargetName, const FString& DestLocalizationTargetDirectory, const EMergeLocalizationMode MergeMode, const bool bOnlyIfSourceFileNewer, const bool bUseSourceControl = true);

/**
 * Cleanup UGC localization that is no longer relevant based on the given descriptor.
 *
 * @param PluginsToClean		The list of plugins to cleanup.
 * @param PluginsToRemove		The list of plugins to remove any localization data for, regardless of the UGC localization descriptor.
 * @param DefaultDescriptor		The default UGC localization descriptor to filter existing UGC localization data against (things that don't pass the filter will be cleaned).
 * @param bSilent				True to silently delete localization data that doesn't pass the filter, or false to confirm with the user (via a dialog).
 */
LOCALIZATION_API void CleanupLocalization(TArrayView<const TSharedRef<IPlugin>> PluginsToClean, const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor, const bool bSilent = false);
LOCALIZATION_API void CleanupLocalization(TArrayView<const TSharedRef<IPlugin>> PluginsToClean, TArrayView<const TSharedRef<IPlugin>> PluginsToRemove, const FUserGeneratedContentLocalizationDescriptor& DefaultDescriptor, const bool bSilent = false);

}

#undef UE_API
